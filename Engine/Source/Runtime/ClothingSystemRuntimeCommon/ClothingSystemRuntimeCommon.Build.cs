// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClothingSystemRuntimeCommon : ModuleRules
{
	public ClothingSystemRuntimeCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		SetupModulePhysicsSupport(Target);

		PublicIncludePathModuleNames.Add("ClothingSystemRuntimeInterface");

		PublicDependencyModuleNames.AddRange(
		new string[] {
			"Core",
			"CoreUObject",
			"ClothingSystemRuntimeInterface"
		}
		);

		PrivateDependencyModuleNames.AddRange(
		new string[]
		{
			"Engine",
			"Slate"
		}
		);
	}
}
