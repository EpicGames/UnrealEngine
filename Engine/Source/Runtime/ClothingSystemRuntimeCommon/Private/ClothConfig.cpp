// Copyright Epic Games, Inc. All Rights Reserved.
#include "ClothConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothConfig)

UClothConfigCommon::UClothConfigCommon()
{}

UClothConfigCommon::~UClothConfigCommon()
{}

UClothSharedConfigCommon::UClothSharedConfigCommon()
{}

UClothSharedConfigCommon::~UClothSharedConfigCommon()
{}

