// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothLODData.h"
#include "ClothPhysicalMeshData.h"
#include "ClothingAssetCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothLODData)

#if WITH_EDITORONLY_DATA
void FClothLODDataCommon::GetParameterMasksForTarget(
	const uint8 InTarget, 
	TArray<FPointWeightMap*>& OutMasks)
{
	for(FPointWeightMap& Mask : PointWeightMaps)
	{
		if(Mask.CurrentTarget == InTarget)
		{
			OutMasks.Add(&Mask);
		}
	}
}
#endif // WITH_EDITORONLY_DATA

bool FClothLODDataCommon::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FClothingAssetCustomVersion::GUID);

	// Serialize normal tagged property data
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		UScriptStruct* const Struct = FClothLODDataCommon::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

	// Serialize the mesh to mesh data (not a USTRUCT)
	Ar << TransitionUpSkinData
	   << TransitionDownSkinData;

#if WITH_EDITORONLY_DATA
	const int32 ClothingCustomVersion = Ar.CustomVer(FClothingAssetCustomVersion::GUID);
	if (Ar.IsLoading() && ClothingCustomVersion < FClothingAssetCustomVersion::MovePropertiesToCommonBaseClasses)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Migrate maps
		PhysicalMeshData.GetWeightMap(EWeightMapTargetCommon::MaxDistance).Values = MoveTemp(PhysicalMeshData.MaxDistances_DEPRECATED);
		PhysicalMeshData.GetWeightMap(EWeightMapTargetCommon::BackstopDistance).Values = MoveTemp(PhysicalMeshData.BackstopDistances_DEPRECATED);
		PhysicalMeshData.GetWeightMap(EWeightMapTargetCommon::BackstopRadius).Values = MoveTemp(PhysicalMeshData.BackstopRadiuses_DEPRECATED);
		PhysicalMeshData.GetWeightMap(EWeightMapTargetCommon::AnimDriveStiffness).Values = MoveTemp(PhysicalMeshData.AnimDriveMultipliers_DEPRECATED);

		// Migrate editor maps
		PointWeightMaps.SetNum(ParameterMasks_DEPRECATED.Num());
		for (int32 i = 0; i < PointWeightMaps.Num(); ++i)
		{
			ParameterMasks_DEPRECATED[i].MigrateTo(PointWeightMaps[i]);
		}
		ParameterMasks_DEPRECATED.Empty();

		// Remove deprecated Apex collisions from the LOD data
		CollisionData_DEPRECATED.Spheres.Empty();
		CollisionData_DEPRECATED.SphereConnections.Empty();
		CollisionData_DEPRECATED.Convexes.Empty();
		CollisionData_DEPRECATED.Boxes.Empty();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITORONLY_DATA
	return true;
}

#if WITH_EDITOR
void FClothLODDataCommon::PushWeightsToMesh()
{
	PhysicalMeshData.ClearWeightMaps();
	for (const FPointWeightMap& Weights : PointWeightMaps)
	{
		if (Weights.bEnabled)
		{
			FPointWeightMap& TargetWeightMap = PhysicalMeshData.FindOrAddWeightMap(Weights.CurrentTarget);
			TargetWeightMap.Values = Weights.Values;
		}
	}
}
#endif

