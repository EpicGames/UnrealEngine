// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothLODData_Legacy.h"
#include "ClothLODData.h"
#include "ClothPhysicalMeshDataBase_Legacy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothLODData_Legacy)

FClothParameterMask_Legacy::FClothParameterMask_Legacy()
	: MaskName(NAME_None)
	, CurrentTarget(EWeightMapTargetCommon::None)
	, MaxValue_DEPRECATED(0.0)
	, MinValue_DEPRECATED(100.0)
	, bEnabled(false)
{}

void FClothParameterMask_Legacy::MigrateTo(FPointWeightMap& PointWeightMap)
{
	PointWeightMap.Values = MoveTemp(Values);
#if WITH_EDITORONLY_DATA
	PointWeightMap.Name = MaskName;
	PointWeightMap.CurrentTarget = static_cast<uint8>(CurrentTarget);
	PointWeightMap.bEnabled = bEnabled;
#endif
}

UClothLODDataCommon_Legacy::UClothLODDataCommon_Legacy(const FObjectInitializer& Init)
	: Super(Init)
	, PhysicalMeshData_DEPRECATED(nullptr)
{
}

UClothLODDataCommon_Legacy::~UClothLODDataCommon_Legacy()
{}

void UClothLODDataCommon_Legacy::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize the mesh to mesh data (not a USTRUCT)
	Ar << TransitionUpSkinData
	   << TransitionDownSkinData;
}

void UClothLODDataCommon_Legacy::PostLoad()
{
	Super::PostLoad();

	if (PhysicalMeshData_DEPRECATED)
	{
		PhysicalMeshData_DEPRECATED->ConditionalPostLoad();  // Makes sure the UObject has finished loading
		ClothPhysicalMeshData.MigrateFrom(PhysicalMeshData_DEPRECATED);
		PhysicalMeshData_DEPRECATED = nullptr;
	}
}

void UClothLODDataCommon_Legacy::MigrateTo(FClothLODDataCommon& LodData)
{
	// Migrate mesh data
	if (PhysicalMeshData_DEPRECATED)
	{
		// Migrate PhysicalMeshData from when it was a UObject
		PhysicalMeshData_DEPRECATED->ConditionalPostLoad();  // Makes sure the UObject has finished loading
		LodData.PhysicalMeshData.MigrateFrom(PhysicalMeshData_DEPRECATED);
	}
	else
	{
		// Migrate PhysicalMeshData from when it was a UStruct
		LodData.PhysicalMeshData.MigrateFrom(ClothPhysicalMeshData);
	}

#if WITH_EDITORONLY_DATA
	// Migrate editor maps
	LodData.PointWeightMaps = MoveTemp(ParameterMasks);
#endif // WITH_EDITORONLY_DATA

	// Migrate skinning data
	LodData.TransitionUpSkinData = MoveTemp(TransitionUpSkinData);
	LodData.TransitionDownSkinData = MoveTemp(TransitionDownSkinData);
}

