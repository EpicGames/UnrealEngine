// Copyright Epic Games, Inc. All Rights Reserved.
#include "ClothPhysicalMeshData.h"
#include "ClothConfigBase.h"
#include "ClothPhysicalMeshDataBase_Legacy.h"
#include "GPUSkinPublicDefs.h"  // For MAX_TOTAL_INFLUENCES
#include "ClothTetherData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothPhysicalMeshData)

FClothPhysicalMeshData::FClothPhysicalMeshData()
	: MaxBoneWeights(0)
	, NumFixedVerts(0)
{
	ClearWeightMaps();
}

void FClothPhysicalMeshData::MigrateFrom(FClothPhysicalMeshData& ClothPhysicalMeshData)
{
	if (this != &ClothPhysicalMeshData)
	{
		Vertices = MoveTemp(ClothPhysicalMeshData.Vertices);
		Normals = MoveTemp(ClothPhysicalMeshData.Normals);
#if WITH_EDITORONLY_DATA
		VertexColors = MoveTemp(ClothPhysicalMeshData.VertexColors);
#endif
		Indices = MoveTemp(ClothPhysicalMeshData.Indices);
		WeightMaps = MoveTemp(ClothPhysicalMeshData.WeightMaps);
		InverseMasses = MoveTemp(ClothPhysicalMeshData.InverseMasses);
		BoneData = MoveTemp(ClothPhysicalMeshData.BoneData);
		NumFixedVerts = ClothPhysicalMeshData.NumFixedVerts;
		MaxBoneWeights = ClothPhysicalMeshData.MaxBoneWeights;
		SelfCollisionVertexSet = MoveTemp(ClothPhysicalMeshData.SelfCollisionVertexSet);
	}
}

void FClothPhysicalMeshData::MigrateFrom(UClothPhysicalMeshDataBase_Legacy* ClothPhysicalMeshDataBase)
{
	Vertices = MoveTemp(ClothPhysicalMeshDataBase->Vertices);
	Normals = MoveTemp(ClothPhysicalMeshDataBase->Normals);
#if WITH_EDITORONLY_DATA
	VertexColors = MoveTemp(ClothPhysicalMeshDataBase->VertexColors);
#endif
	Indices = MoveTemp(ClothPhysicalMeshDataBase->Indices);
	InverseMasses = MoveTemp(ClothPhysicalMeshDataBase->InverseMasses);
	BoneData = MoveTemp(ClothPhysicalMeshDataBase->BoneData);
	NumFixedVerts = ClothPhysicalMeshDataBase->NumFixedVerts;
	MaxBoneWeights = ClothPhysicalMeshDataBase->MaxBoneWeights;
	SelfCollisionVertexSet.Empty(ClothPhysicalMeshDataBase->SelfCollisionIndices.Num());
	for (uint32 Index : ClothPhysicalMeshDataBase->SelfCollisionIndices)
	{
		SelfCollisionVertexSet.Add((int32)Index);
	}

	const TArray<uint32> FloatArrayIds = ClothPhysicalMeshDataBase->GetFloatArrayIds();
	for (uint32 FloatArrayId : FloatArrayIds)
	{
		if (TArray<float>* const FloatArray = ClothPhysicalMeshDataBase->GetFloatArray(FloatArrayId))
		{
			FindOrAddWeightMap(FloatArrayId).Values = MoveTemp(*FloatArray);
		}
	}
}

void FClothPhysicalMeshData::Reset(const int32 InNumVerts, const int32 InNumIndices)
{
	Vertices.Init(FVector3f::ZeroVector, InNumVerts);
	Normals.Init(FVector3f::ZeroVector, InNumVerts);
#if WITH_EDITORONLY_DATA
	VertexColors.Init(FColor::Black, InNumVerts);
#endif //#if WITH_EDITORONLY_DATA
	InverseMasses.Init(0.f, InNumVerts);
	BoneData.Reset(InNumVerts);
	BoneData.AddDefaulted(InNumVerts);
	Indices.Init(0, InNumIndices);

	SelfCollisionVertexSet.Reset();

	NumFixedVerts = 0;
	MaxBoneWeights = 0;

	ClearWeightMaps();
}

void FClothPhysicalMeshData::ClearWeightMaps()
{
	// Clear all weight maps (and reserve a few slots)
	WeightMaps.Empty(4);

	// Add default (empty) optional maps, as these are always expected to be found
	AddWeightMap(EWeightMapTargetCommon::MaxDistance);
	AddWeightMap(EWeightMapTargetCommon::BackstopDistance);
	AddWeightMap(EWeightMapTargetCommon::BackstopRadius);
	AddWeightMap(EWeightMapTargetCommon::AnimDriveStiffness);
}

void FClothPhysicalMeshData::BuildSelfCollisionData(float SelfCollisionRadius)
{
	const float SelfCollisionDiamSq = 4.f *SelfCollisionRadius * SelfCollisionRadius;

	// Start with the full set
	const int32 NumVerts = Vertices.Num();
	TArray<bool> VertexIsValid;
	VertexIsValid.Init(true, NumVerts);

	const FPointWeightMap& MaxDistances = GetWeightMap(EWeightMapTargetCommon::MaxDistance);
	for (int32 Index = 0; Index < NumVerts; ++Index)
	{
		if (MaxDistances.IsBelowThreshold(Index))
		{
			VertexIsValid[Index] = false;
		}
	}

	// Now start aggressively culling verts that are near others that we have accepted
	SelfCollisionVertexSet.Reset();
	for (int32 V0Index = 0; V0Index < NumVerts; ++V0Index)
	{
		if (!VertexIsValid[V0Index])
		{
			continue;
		}
		SelfCollisionVertexSet.Add(V0Index);

		const FVector& V0Pos = (FVector)Vertices[V0Index];

		// Start one after our current V0, we've done the other checks
		for (int32 V1Index = V0Index + 1; V1Index < NumVerts; ++V1Index)
		{
			if (!VertexIsValid[V1Index])
			{
				continue;
			}

			const FVector& V1Pos = (FVector)Vertices[V1Index];
			const float V0ToV1DistSq = (V1Pos - V0Pos).SizeSquared();
			if (V0ToV1DistSq < SelfCollisionDiamSq)
			{
				// Points are in contact in the rest state.  Remove it.
				//
				// It's worth noting that this biases towards removing indices 
				// of later in the list, and keeping ones earlier.  That's not
				// a great criteria for choosing which one is more important.
				VertexIsValid[V1Index] = false;
				continue;
			}
		}
	}
}

void FClothPhysicalMeshData::CalculateInverseMasses()
{
	// Recalculate inverse masses for the physical mesh particles
	check(Indices.Num() % 3 == 0);

	const int32 NumVerts = Vertices.Num();
	InverseMasses.Empty(NumVerts);
	InverseMasses.AddZeroed(NumVerts);

	for (int32 TriBaseIndex = 0; TriBaseIndex < Indices.Num(); TriBaseIndex += 3)
	{
		const int32 Index0 = Indices[TriBaseIndex];
		const int32 Index1 = Indices[TriBaseIndex + 1];
		const int32 Index2 = Indices[TriBaseIndex + 2];

		const FVector AB = FVector(Vertices[Index1] - Vertices[Index0]);
		const FVector AC = FVector(Vertices[Index2] - Vertices[Index0]);
		const float TriArea = FVector::CrossProduct(AB, AC).Size();

		InverseMasses[Index0] += TriArea;
		InverseMasses[Index1] += TriArea;
		InverseMasses[Index2] += TriArea;
	}

	NumFixedVerts = 0;

	const FPointWeightMap* const MaxDistances = FindWeightMap(EWeightMapTargetCommon::MaxDistance);
	const TFunction<bool(int32)> IsKinematic = (!MaxDistances || !MaxDistances->Num()) ?
		TFunction<bool(int32)>([](int32)->bool { return false; }) :
		TFunction<bool(int32)>([&MaxDistances](int32 Index)->bool { return (*MaxDistances)[Index] < SMALL_NUMBER; });  // For consistency, the default Threshold should be 0.1, not SMALL_NUMBER. But for backward compatibility it needs to be SMALL_NUMBER for now.

	float MassSum = 0.0f;
	for (int32 CurrVertIndex = 0; CurrVertIndex < NumVerts; ++CurrVertIndex)
	{
		float& InverseMass = InverseMasses[CurrVertIndex];

		if (IsKinematic(CurrVertIndex))
		{
			InverseMass = 0.0f;
			++NumFixedVerts;
		}
		else
		{
			MassSum += InverseMass;
		}
	}

	if (MassSum > 0.0f)
	{
		const float MassScale = (float)(NumVerts - NumFixedVerts) / MassSum;
		for (float& InverseMass : InverseMasses)
		{
			if (InverseMass != 0.0f)
			{
				InverseMass *= MassScale;
				InverseMass = 1.0f / InverseMass;
			}
		}
	}
	// TODO: Cache config base (Chaos) mass data
	//       Note that multiple configs with different mass modes will require a different
	//       structure than this one to store the mass.
}

void FClothPhysicalMeshData::CalculateNumInfluences()
{
	// Needed by all cloth implementations got skinning
	for (int32 VertIndex = 0; VertIndex < Vertices.Num(); ++VertIndex)
	{
		FClothVertBoneData& BoneDatum = BoneData[VertIndex];
		const uint16* BoneIndices = BoneDatum.BoneIndices;
		const float* BoneWeights = BoneDatum.BoneWeights;

		BoneDatum.NumInfluences = MAX_TOTAL_INFLUENCES;

		int32 NumInfluences = 0;
		for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
		{
			if (BoneWeights[InfluenceIndex] == 0.0f || BoneIndices[InfluenceIndex] == INDEX_NONE)
			{
				BoneDatum.NumInfluences = NumInfluences;
				break;
			}
			++NumInfluences;
		}
	}
}

void FClothPhysicalMeshData::CalculateTethers(bool bUseEuclideanDistance, bool bUseGeodesicDistance)
{
	const FPointWeightMap* const MaxDistances = FindWeightMap(EWeightMapTargetCommon::MaxDistance);
	const FPointWeightMap* const TetherEnds = FindWeightMap(EWeightMapTargetCommon::TetherEndsMask);

	if (TetherEnds)
	{
		// Note: Technically there could be two different configs requiring different flavour of tethers,
		//       in reality at the time of writing there is only one type of tether involved at any one type.
		if (bUseEuclideanDistance)
		{
			constexpr bool bGenerateGeodesic = false;
			EuclideanTethers.GenerateTethers(Vertices, Indices, TetherEnds->Values, bGenerateGeodesic);
		}
		if (bUseGeodesicDistance)
		{
			constexpr bool bGenegateGeodesic = true;
			GeodesicTethers.GenerateTethers(Vertices, Indices, TetherEnds->Values, bGenegateGeodesic);
		}

	}
	else if (MaxDistances)
	{
		// Note: Technically there could be two different configs requiring different flavour of tethers,
		//       in reality at the time of writing there is only one type of tether involved at any one type.
		if (bUseEuclideanDistance)
		{
			constexpr bool bGenerateGeodesic = false;
			EuclideanTethers.GenerateTethers(Vertices, Indices, MaxDistances->Values, bGenerateGeodesic);
		}
		if (bUseGeodesicDistance)
		{
			constexpr bool bGenegateGeodesic = true;
			GeodesicTethers.GenerateTethers(Vertices, Indices, MaxDistances->Values, bGenegateGeodesic);
		}
	}
}
