// Copyright Epic Games, Inc. All Rights Reserved.
#include "ClothTetherData.h"
#include "Async/ParallelFor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothTetherData)

class FClothTetherDataPrivate
{
public:
	static constexpr int32 MaxNumAttachments = 4;  // Max recommended number of tethers per point, could eventually been specified by config
	// Re-export the public threshold so the existing FClothTetherDataPrivate::KinematicDistanceThreshold
	// usages below stay valid without forcing the local consumers to qualify with FClothTetherData::.
	static constexpr float KinematicDistanceThreshold = FClothTetherData::KinematicDistanceThreshold;

	typedef TTuple<int32, int32, float> FTether;

	FClothTetherDataPrivate(
		const TConstArrayView<FVector3f>& Points,
		const TConstArrayView<uint32>& Indices,
		TFunctionRef<bool(int32 Index)> IsKinematic,
		TFunctionRef<bool(int32 Index)> IsDynamicTetherEnd,
		TFunctionRef<bool(int32 Index)> IsKinematicTetherEnd,
		bool bUseGeodesicDistance);

	FClothTetherDataPrivate(
		TArray<TArray<TPair<float, int32>>>&& PerDynamicNodeTethers);

	void GetBatchedTetherData(TArray<TArray<TTuple<int32, int32, float>>>& Tethers) const;

private:

	// Generate an array of unconnected islands from a selection of points.
	void ComputeKinematicNodeIslands();

	// Generate a map of tethers by following the triangle mesh network
	// The choice of the tether is determined by the shortest euclidean (beeline) distance.
	void GenerateEuclideanTethers(const TConstArrayView<FVector3f>& Points);

	// Generate a map of tethers by following the triangle mesh network
	// The choice of the tether is determined by the shortest geodesic (curvature) distance.
	void GenerateGeodesicTethers(const TConstArrayView<FVector3f>& Points,
		TFunctionRef<bool(int32 Index)> IsKinematic);

	// Update TetherNums after computing tethers.
	void UpdateCounts();

private:
	TMap<int32, TSet<int32>> NodeToNeighbors;
	TArray<int32> AllDynamicNodes;
	TArray<int32> DynamicTetherNodesDynamicIndex; // Index into AllDynamicNodes, not Nodes.
	TArray<int32> KinematicTetherNodes;
	TArray<TArray<int32>> KinematicNodeIslands;
	TArray<TArray<FTether>> TetherSlots;  // Each dynamic node has between 0 and MaxNumAttachments slots
	TArray<int32> TetherNums;
};

void FClothTetherData::GenerateTethers(
	const TConstArrayView<FVector3f>& Points,
	const TConstArrayView<uint32>& Indices,
	const TConstArrayView<float>& MaxDistances,
	bool bUseGeodesicDistance)
{
	// Early exit if there is no MaxDistances mask
	if (Points.Num() != MaxDistances.Num())
	{
		Tethers.Empty();
		return;
	}

	// Calculate the tethers
	const FClothTetherDataPrivate ClothTetherData(Points, Indices,
		[&MaxDistances](int32 Node)
		{
			return MaxDistances[Node] < FClothTetherDataPrivate::KinematicDistanceThreshold;
		},
		[&MaxDistances](int32 Node)
		{
			return MaxDistances[Node] >= FClothTetherDataPrivate::KinematicDistanceThreshold;
		},
		[&MaxDistances](int32 Node)
		{
			return MaxDistances[Node] < FClothTetherDataPrivate::KinematicDistanceThreshold;
		},
		bUseGeodesicDistance);
	ClothTetherData.GetBatchedTetherData(Tethers);
}

void FClothTetherData::GenerateTethers(
	const TConstArrayView<FVector3f>& Points,
	const TConstArrayView<uint32>& Indices,
	const TConstArrayView<float>& MaxDistances,
	const TConstArrayView<float>& TetherEnds,
	bool bUseGeodesicDistance)
{
	// Early exit if there is no MaxDistances mask
	if (Points.Num() != MaxDistances.Num())
	{
		Tethers.Empty();
		return;
	}

	// Calculate the tethers
	const FClothTetherDataPrivate ClothTetherData(Points, Indices,
		[&MaxDistances](int32 Node)
		{
			return MaxDistances[Node] < FClothTetherDataPrivate::KinematicDistanceThreshold;
		},
		[&MaxDistances](int32 Node)
		{
			return MaxDistances[Node] >= FClothTetherDataPrivate::KinematicDistanceThreshold;
		},
		[&TetherEnds](int32 Node)
		{
			return TetherEnds[Node] < FClothTetherDataPrivate::KinematicDistanceThreshold;
		},
		bUseGeodesicDistance);
	ClothTetherData.GetBatchedTetherData(Tethers);
}

void FClothTetherData::GenerateTethers(
	const TConstArrayView<FVector3f>& Points,
	const TConstArrayView<uint32>& Indices,
	const TSet<int32>& KinematicNodes,
	bool bUseGeodesicDistance)
{
	// Early exit if there is no KinematicNodes set
	if (KinematicNodes.IsEmpty())
	{
		Tethers.Empty();
		return;
	}

	// Calculate the tethers
	const FClothTetherDataPrivate ClothTetherData(Points, Indices,
		[&KinematicNodes](int32 Node)
		{
			return KinematicNodes.Contains(Node);
		},
		[&KinematicNodes](int32 Node)
		{
			return !KinematicNodes.Contains(Node);
		},
		[&KinematicNodes](int32 Node)
		{
			return KinematicNodes.Contains(Node);
		},
		bUseGeodesicDistance);
	ClothTetherData.GetBatchedTetherData(Tethers);
}

void FClothTetherData::GenerateTethers(
	const TConstArrayView<FVector3f>& Points,  // Reference pose
	const TConstArrayView<uint32>& Indices,  // Triangle mesh
	const TSet<int32>& KinematicNodes,  // all kinematic indices
	const TSet<int32>& FixedEndNodes,  // subset of kinematic indices used for fixed end.
	const TSet<int32>& DynamicEndNodes,  // subset of dynamic indices to create tethers for.
	bool bUseGeodesicDistance)
{
	// Early exit if there is no KinematicNodes or DynamicNodes set
	if (FixedEndNodes.IsEmpty() || DynamicEndNodes.IsEmpty() || KinematicNodes.IsEmpty())
	{
		Tethers.Empty();
		return;
	}

	// Calculate the tethers
	const FClothTetherDataPrivate ClothTetherData(Points, Indices,
		[&KinematicNodes](int32 Node)
		{
			return KinematicNodes.Contains(Node);
		},
		[&KinematicNodes, &DynamicEndNodes](int32 Node)
		{
			return DynamicEndNodes.Contains(Node) && !KinematicNodes.Contains(Node);
		},
		[&KinematicNodes, &FixedEndNodes](int32 Node)
		{
			return FixedEndNodes.Contains(Node) && KinematicNodes.Contains(Node);
		},
		bUseGeodesicDistance);
	ClothTetherData.GetBatchedTetherData(Tethers);

}

void FClothTetherData::GenerateTethers(
	TArray<TArray<TPair<float, int32>>>&& PerDynamicNodeTethers)
{
	const FClothTetherDataPrivate ClothTetherData(MoveTemp(PerDynamicNodeTethers));
	ClothTetherData.GetBatchedTetherData(Tethers);
}

bool FClothTetherData::Serialize(FArchive& Ar)
{
	// Serialize normal tagged property data
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		UScriptStruct* const Struct = FClothTetherData::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

	// Serialize the tethers
	Ar << Tethers;

	return true;
}

FClothTetherDataPrivate::FClothTetherDataPrivate(
	const TConstArrayView<FVector3f>& Points,
	const TConstArrayView<uint32>& Indices,
	TFunctionRef<bool(int32 Index)> IsKinematic,
	TFunctionRef<bool(int32 Index)> IsDynamicTetherEnd,
	TFunctionRef<bool(int32 Index)> IsKinematicTetherEnd,
	bool bUseGeodesicDistance)
{
	// Calculate the points' neighbors map
	NodeToNeighbors.Reserve(Points.Num());

	for (int32 i = 0; i < Indices.Num(); i += 3)
	{
		const int32 Index0 = Indices[i];
		const int32 Index1 = Indices[i + 1];
		const int32 Index2 = Indices[i + 2];

		NodeToNeighbors.FindOrAdd(Index0).Append({ Index1, Index2 });
		NodeToNeighbors.FindOrAdd(Index1).Append({ Index0, Index2 });
		NodeToNeighbors.FindOrAdd(Index2).Append({ Index0, Index1 });
	}

	// Fill up the list of all used indices
	TArray<int32> Nodes;
	NodeToNeighbors.GenerateKeyArray(Nodes);

	// Find all kinematic points to use as anchor points for the tethers and seed the path finding
	AllDynamicNodes.Reserve(Points.Num());
	DynamicTetherNodesDynamicIndex.Reserve(Points.Num());
	KinematicTetherNodes.Reserve(Points.Num());
	for (const int32 Node : Nodes)
	{
		if (!IsKinematic(Node))
		{
			const int32 DynamicIndex = AllDynamicNodes.Add(Node);
			if (IsDynamicTetherEnd(Node))
			{
				DynamicTetherNodesDynamicIndex.Add(DynamicIndex);
			}

		}
		else if (IsKinematicTetherEnd(Node))
		{
			KinematicTetherNodes.Add(Node);
		}
	}

	if (KinematicTetherNodes.Num())
	{
		// Compute the islands of kinematic particles
		ComputeKinematicNodeIslands();

		// Allocate the tether batches, each node has a maximum of MaxNumAttachments slots
		TetherSlots.SetNum(DynamicTetherNodesDynamicIndex.Num());

		for (TArray<FTether>& TetherSlot : TetherSlots)
		{
			TetherSlot.Reserve(MaxNumAttachments);
		}

		// Find the tethers
		if (!bUseGeodesicDistance)
		{
			GenerateEuclideanTethers(Points);
		}
		else
		{
			GenerateGeodesicTethers(Points, IsKinematic);
		}

		UpdateCounts();
	}
}

FClothTetherDataPrivate::FClothTetherDataPrivate(TArray<TArray<TPair<float, int32>>>&& PerDynamicNodeTethers)
{
	TetherSlots.SetNum(PerDynamicNodeTethers.Num());

	// For each dynamic node
	ParallelFor(PerDynamicNodeTethers.Num(), [this, &PerDynamicNodeTethers](int32 Index)
	{
		const int32 DynamicNode = Index;
		TArray<TPair<float, int32>>& ClosestKinematicNodes = PerDynamicNodeTethers[Index];
		// Only keep the first MaxNumAttachments closest kinematic nodes
		if (ClosestKinematicNodes.Num() > MaxNumAttachments)
		{
			// Order all by distance, smallest first
			ClosestKinematicNodes.Sort();

			// Shrink the list... but not the array
			ClosestKinematicNodes.SetNum(MaxNumAttachments, EAllowShrinking::No);
		}
		// Finally create the tethers between this dynamic node and the N closests kinematic ones
		TetherSlots[Index].Reserve(ClosestKinematicNodes.Num());

		for (const TPair<float, int32> ClosestKinematicNode : ClosestKinematicNodes)
		{
			const float Length = ClosestKinematicNode.Key;
			const int32 KinematicNode = ClosestKinematicNode.Value;

			TetherSlots[Index].Emplace(KinematicNode, DynamicNode, Length);
		}
	});

	UpdateCounts();
}

void FClothTetherDataPrivate::GetBatchedTetherData(TArray<TArray<TTuple<int32, int32, float>>>& Tethers) const
{
	// Reorganize the multiple tethers per node array into a single sequential batches of tethers more suitable for parallel processing
	const int32 NumDynamicNodes = TetherSlots.Num();
	const int32 MaxNumUsedSlots = TetherNums.Num();
	Tethers.Reset(MaxNumUsedSlots);
	Tethers.SetNum(MaxNumUsedSlots);

	for (int32 Slot = 0; Slot < MaxNumUsedSlots; ++Slot)
	{
		Tethers[Slot].Reserve(TetherNums[Slot]);

		for (int32 Index = 0; Index < NumDynamicNodes; ++Index)
		{
			if (TetherSlots[Index].IsValidIndex(Slot))
			{
				Tethers[Slot].Emplace(TetherSlots[Index][Slot]);
			}
		}
	}
}

void FClothTetherDataPrivate::ComputeKinematicNodeIslands()
{
	KinematicNodeIslands.Reset();

	int32 NextIslandIndex = 0;
	TArray<int32> FreeIslandIndices;

	TMap<int32, int32> NodeToIslands;
	NodeToIslands.Reserve(KinematicTetherNodes.Num());

	for (const int32 KinematicNode : KinematicTetherNodes)
	{
		// Assign this point an island, possibly unionizing existing islands
		int32 IslandIndex = TNumericLimits<int32>::Max();

		const TSet<int32>& NeighborNodes = NodeToNeighbors[KinematicNode];
		for (const int32 NeighborNode : NeighborNodes)
		{
			if (const int32* const NeighborIslandIndexPtr = NodeToIslands.Find(NeighborNode))
			{
				const int32 NeighborIslandIndex = *NeighborIslandIndexPtr;

				if (IslandIndex == TNumericLimits<int32>::Max())
				{
					// No assigned island yet, join with the neighbor's island
					IslandIndex = NeighborIslandIndex;
				}
				else if (NeighborIslandIndex != IslandIndex)
				{
					// This point is connected to multiple islands, so union them
					TArray<int32>& NeighborIsland = KinematicNodeIslands[NeighborIslandIndex];
					for (const int32 NeighborIslandNode : NeighborIsland)
					{
						check(NodeToIslands[NeighborIslandNode] == NeighborIslandIndex);
						NodeToIslands[NeighborIslandNode] = IslandIndex;
					}
					KinematicNodeIslands[IslandIndex].Append(NeighborIsland); // Union the two islands
					NeighborIsland.Reset();  // Don't deallocate, to allow for reusing the free island
					FreeIslandIndices.AddUnique(NeighborIslandIndex);
				}
				// Else this neighbor is already in the same island as the previous neighbors
			}
		}

		// If no connected IslandIndex was found, create a new one (or reuse an old vacated one)
		if (IslandIndex == TNumericLimits<int32>::Max())
		{
			if (!FreeIslandIndices.Num())
			{
				IslandIndex = NextIslandIndex++;
				KinematicNodeIslands.SetNum(NextIslandIndex);
			}
			else
			{
				// Reuse a previously allocated, but currently unused, island
				IslandIndex = FreeIslandIndices.Pop();
			}
		}

		NodeToIslands.FindOrAdd(KinematicNode) = IslandIndex;
		check(KinematicNodeIslands.IsValidIndex(IslandIndex));
		KinematicNodeIslands[IslandIndex].Add(KinematicNode);
	}

	// Remove empty KinematicNodeIslands
	int32 TotalNumIslandKinematicNodes = 0;
	for (int32 IslandIndex = 0; IslandIndex < KinematicNodeIslands.Num(); )
	{
		const int32 NumIslandKinematicNodes = KinematicNodeIslands[IslandIndex].Num();
		if (!NumIslandKinematicNodes)
		{
			KinematicNodeIslands.RemoveAtSwap(IslandIndex, EAllowShrinking::No);
			// RemoveAtSwap takes the last elements to replace the current one, do not increment the index in this case
		}
		else
		{
			TotalNumIslandKinematicNodes += NumIslandKinematicNodes;
			++IslandIndex;
		}
	}

	// Final sanity check
	check(TotalNumIslandKinematicNodes == KinematicTetherNodes.Num());
}

void FClothTetherDataPrivate::GenerateEuclideanTethers(const TConstArrayView<FVector3f>& Points)
{
	check(KinematicNodeIslands.Num());

	// For each dynamic node
	ParallelFor(DynamicTetherNodesDynamicIndex.Num(), [this, &Points](int32 Index)
	{
		const int32 DynamicNode = AllDynamicNodes[DynamicTetherNodesDynamicIndex[Index]];

		// Measure the distance to all kinematic nodes, and keep the closest from each island
		TArray<TPair<float, int32>> ClosestKinematicNodes;
		ClosestKinematicNodes.Reserve(KinematicNodeIslands.Num());

		for (const TArray<int32>& KinematicNodeIsland : KinematicNodeIslands)
		{
			check(KinematicNodeIsland.Num());  // Island must not be an empty array

			int32 ClosestKinematicNode = TNumericLimits<int32>::Max();
			float ClosestSquareDistance = TNumericLimits<float>::Max();
			for (const int32 KinematicNode : KinematicNodeIsland)
			{
				const float SquareDistance = (Points[KinematicNode] - Points[DynamicNode]).SizeSquared();
				if (SquareDistance < ClosestSquareDistance)
				{
					ClosestKinematicNode = KinematicNode;
					ClosestSquareDistance = SquareDistance;
				}
			}
			check(ClosestKinematicNode != TNumericLimits<int32>::Max());

			ClosestKinematicNodes.Emplace(FMath::Sqrt(ClosestSquareDistance), ClosestKinematicNode);
		}

		// Only keep the first MaxNumAttachments closest kinematic nodes
		if (ClosestKinematicNodes.Num() > MaxNumAttachments)
		{
			// Order all by distance, smallest first
			ClosestKinematicNodes.Sort();

			// Shrink the list... but not the array
			ClosestKinematicNodes.SetNum(MaxNumAttachments, EAllowShrinking::No);
		}

		// Finally create the tethers between this dynamic node and the N closests kinematic ones
		TetherSlots[Index].Reserve(ClosestKinematicNodes.Num());

		for (const TPair<float, int32> ClosestKinematicNode : ClosestKinematicNodes)
		{
			const float Length = ClosestKinematicNode.Key;
			const int32 KinematicNode = ClosestKinematicNode.Value;

			TetherSlots[Index].Emplace(KinematicNode, DynamicNode, Length);
		}
	});
}

void FClothTetherDataPrivate::GenerateGeodesicTethers(const TConstArrayView<FVector3f>& Points,
	TFunctionRef<bool(int32 Index)> IsKinematic)
{
	check(KinematicNodeIslands.Num());

	// Find all seeds in each island, kinematic tether end nodes connected to at least one non-tether end
	int32 NumSeeds = 0;
	TArray<TArray<int32>> SeedIslands;
	SeedIslands.SetNum(KinematicNodeIslands.Num());

	for (int32 IslandIndex = 0; IslandIndex < KinematicNodeIslands.Num(); ++IslandIndex)
	{
		const TArray<int32>& KinematicNodeIsland = KinematicNodeIslands[IslandIndex];
		SeedIslands[IslandIndex].Reserve(KinematicNodeIsland.Num());

		for (const int32 KinematicNode : KinematicNodeIsland)
		{
			const TSet<int32>& Neighbors = NodeToNeighbors[KinematicNode];
			for (const int32 Neighbor : Neighbors)
			{
				if (!IsKinematic(Neighbor))
				{
					SeedIslands[IslandIndex].Add(KinematicNode);
					++NumSeeds;
					break;
				}
			}
		}
	}

	// Consolidate into a single array for ease of iteration
	TArray<int32> Seeds;
	Seeds.Reserve(NumSeeds);

	for (const TArray<int32>& SeedIsland : SeedIslands)
	{
		Seeds.Append(SeedIsland);
	}
	check(Seeds.Num() == NumSeeds);

	// Dijkstra for each Kinematic Particle (assume a small number of kinematic points) - note this is N^2 log N with N kinematic points
	// Find shortest paths from each seed node to all dynamic nodes
	TMap<int32, TMap<int32, float>> GeodesicDistances;  // Uses two maps instead of one map addressed by Seed and Node to avoid data races
	GeodesicDistances.Reserve(Seeds.Num());

	for (const int32 Seed : Seeds)
	{
		TMap<int32, float>& SeedGeodesicDistances = GeodesicDistances.Emplace(Seed);
		SeedGeodesicDistances.Reserve(AllDynamicNodes.Num());

		for (const int32 DynamicNode : AllDynamicNodes)
		{
			SeedGeodesicDistances.Emplace(DynamicNode, TNumericLimits<float>::Max());
		}
	}

	ParallelFor(Seeds.Num(), [this, &Seeds, &GeodesicDistances, &Points, &IsKinematic](int32 Index)
	{
		const int32 Seed = Seeds[Index];
		TMap<int32, float>& SeedGeodesicDistances = GeodesicDistances[Seed];

		// Keep track of all visited nodes in a bit array
		TBitArray<> VisitedNodes(false, Points.Num());

		// Priority queue based implementation of the Dijkstra algorithm
#define CLOTHTETHERDATA_PROFILE_HEAPSIZE 0
#if CLOTHTETHERDATA_PROFILE_HEAPSIZE
		static int32 MaxHeapSize = 1;  // Record the max heap size
#else
		static const int32 MaxHeapSize = 512;  // Set the queue size to something large enough to avoid reallocations in most cases
#endif
		auto LessPredicate = [&SeedGeodesicDistances](int32 Node1, int32 Node2) -> bool
			{
				return SeedGeodesicDistances[Node1] < SeedGeodesicDistances[Node2];  // Less for node priority
			};

		TArray<int32> Queue;
		Queue.Reserve(MaxHeapSize);
		Queue.Heapify(LessPredicate);  // Turn the array into a priority queue

		// Initiate the graph progression
		VisitedNodes[Seed] = true;
		Queue.HeapPush(Seed, LessPredicate);

		do
		{
			int32 ParentNode;
			Queue.HeapPop(ParentNode, LessPredicate, EAllowShrinking::No);

			check(VisitedNodes[ParentNode]);

			const float ParentDistance = (ParentNode != Seed) ? SeedGeodesicDistances[ParentNode] : 0.f;

			const TSet<int32>& NeighborNodes = NodeToNeighbors[ParentNode];
			for (const int32 NeighborNode : NeighborNodes)
			{
				check(NeighborNode != ParentNode);

				// Do not progress onto kinematic nodes
				if (IsKinematic(NeighborNode))
				{
					continue;
				}

				// Update the geodesic distance if this path is a shorter one
				const float NewDistance = ParentDistance + (Points[NeighborNode] - Points[ParentNode]).Size();

				float& GeodesicDistance = SeedGeodesicDistances[NeighborNode];

				if (NewDistance < GeodesicDistance)
				{
					// Update this path distance
					GeodesicDistance = NewDistance;

					// Progress to this node position if it hasn't yet been visited
					if (!VisitedNodes[NeighborNode])
					{
						VisitedNodes[NeighborNode] = true;

						Queue.HeapPush(NeighborNode, LessPredicate);

#if CLOTHTETHERDATA_PROFILE_HEAPSIZE
						MaxHeapSize = FMath::Max(Queue.Num(), MaxHeapSize);
#endif
					}
				}
			}
		} while (Queue.Num());
	});

	// Initialize the tether slot arrays
	TetherSlots.SetNum(DynamicTetherNodesDynamicIndex.Num());

	for (TArray<FTether>& NewTethers : TetherSlots)
	{
		NewTethers.Reserve(MaxNumAttachments);
	}

	// Find the tether constraints starting from each dynamic node
	ParallelFor(DynamicTetherNodesDynamicIndex.Num(), [this, &SeedIslands, &GeodesicDistances](int32 Index)
	{
		const int32 DynamicNode = AllDynamicNodes[DynamicTetherNodesDynamicIndex[Index]];

		// Find the closest seeds in each island
		TArray<int32, TInlineAllocator<32>> ClosestSeeds;
		ClosestSeeds.Reserve(KinematicNodeIslands.Num());

		for (const TArray<int32>& SeedIsland : SeedIslands)
		{
			int32 ClosestSeed = INDEX_NONE;
			float ClosestDistance = TNumericLimits<float>::Max();

			for (const int32 Seed : SeedIsland)
			{
				const float Distance = GeodesicDistances[Seed][DynamicNode];
				if (Distance < ClosestDistance)
				{
					ClosestDistance = Distance;
					ClosestSeed = Seed;
				}
			}
			if (ClosestSeed != INDEX_NONE)
			{
				ClosestSeeds.Add(ClosestSeed);
			}
		}

		// Sort all the tethers for this node based on smallest distance
		ClosestSeeds.Sort([&GeodesicDistances, DynamicNode](int32 Seed1, int32 Seed2)
			{
				return GeodesicDistances[Seed1][DynamicNode] < GeodesicDistances[Seed2][DynamicNode];
			});

		// Keep only N closest tethers
		if (ClosestSeeds.Num() > MaxNumAttachments)
		{
			ClosestSeeds.SetNum(MaxNumAttachments);
		}

		// Add these tethers to the N (or less) available slots
		for (const int32 Seed : ClosestSeeds)
		{
			const float RefLength = GeodesicDistances[Seed][DynamicNode];
			TetherSlots[Index].Emplace(Seed, DynamicNode, RefLength);
		}
		check(TetherSlots[Index].Num() <= MaxNumAttachments);
	});
}

void FClothTetherDataPrivate::UpdateCounts()
{
	// Update counts
	TetherNums.Reserve(MaxNumAttachments);
	for (const TArray<FTether>& TetherSlot : TetherSlots)
	{
		// Resize the count array whenever needed
		const int32 NumUsedSlots = TetherSlot.Num();
		TetherNums.SetNum(FMath::Max(TetherNums.Num(), NumUsedSlots));

		// Increment each used slot count
		for (int32 UsedSlot = 0; UsedSlot < NumUsedSlots; ++UsedSlot)
		{
			++TetherNums[UsedSlot];
		}
	}
}