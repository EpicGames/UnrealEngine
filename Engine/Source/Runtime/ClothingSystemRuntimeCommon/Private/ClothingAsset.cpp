// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingAsset.h"
#include "ClothingAssetCustomVersion.h"
#include "ClothPhysicalMeshData.h"
#include "ClothConfig.h"

#include "Utils/ClothingMeshUtils.h"
#include "Features/IModularFeatures.h"

#include "Engine/SkeletalMesh.h"

#include "PhysicsEngine/PhysicsAsset.h"

#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"

#include "Components/SkeletalMeshComponent.h"

#include "ClothingSimulationFactory.h"
#include "ClothingSimulationInteractor.h"
#include "ComponentReregisterContext.h"
#include "UObject/UObjectIterator.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/UE5SpecialProjectStreamObjectVersion.h"

#include "GPUSkinPublicDefs.h"
#include "GPUSkinVertexFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingAsset)

#define LOCTEXT_NAMESPACE "ClothingAsset"

//==============================================================================
// ClothingAssetUtils
//==============================================================================

#if WITH_EDITOR
void ClothingAssetUtils::UnbindFromSkeletalMesh(const UClothingAssetBase& Asset, USkeletalMesh& SkeletalMesh, const int32 MeshLodIndex, const int32 LodSectionIndex)
{
	FSkeletalMeshModel* const Mesh = SkeletalMesh.GetImportedModel();
	if (!Mesh)
	{
		UE_LOGF(LogClothingAsset, Warning,
			"UnbindFromSkeletalMesh: Failed to unbind Clothing Data [%ls] from Skeletal Mesh [%ls], the Skeletal Mesh imported model doesn't exist.",
			*Asset.GetName(),
			*SkeletalMesh.GetName());
		return;
	}

	int32 StartMeshLodIndex = 0;
	int32 EndMeshLodIndex = -1;
	if (MeshLodIndex == INDEX_NONE)
	{
		EndMeshLodIndex += Mesh->LODModels.Num();
	}
	else if (Mesh->LODModels.IsValidIndex(MeshLodIndex))
	{
		StartMeshLodIndex = EndMeshLodIndex = MeshLodIndex;
	}
	else
	{
		UE_LOGF(LogClothingAsset, Warning,
			"UnbindFromSkeletalMesh: Failed to unbind Clothing Data [%ls] from Skeletal Mesh [%ls], LOD%d doesn't exist.",
			*Asset.GetName(),
			*SkeletalMesh.GetName(),
			MeshLodIndex);
		return;
	}

	bool bChangedMesh = false;
	ON_SCOPE_EXIT  // If the mesh changed we need to re-register any components that use it to reflect the changes
		{
			if (bChangedMesh)
			{
				FScopedSkeletalMeshPostEditChange SkeletalMeshPostEditChange(&SkeletalMesh);
			}
		};
	for (int32 LodIndex = StartMeshLodIndex; LodIndex <= EndMeshLodIndex; ++LodIndex)
	{
		FSkeletalMeshLODModel& LodModel = Mesh->LODModels[LodIndex];

		int32 StartLodSectionIndex = 0;
		int32 EndLodSectionIndex = -1;

		if (MeshLodIndex == INDEX_NONE || LodSectionIndex == INDEX_NONE)
		{
			EndLodSectionIndex += LodModel.Sections.Num();
		}
		else if (LodModel.Sections.IsValidIndex(LodSectionIndex))
		{
			StartLodSectionIndex = EndLodSectionIndex = LodSectionIndex;
		}
		else
		{
			UE_LOGF(LogClothingAsset, Warning,
				"UnbindFromSkeletalMesh: Failed to unbind Clothing Data [%ls] from Skeletal Mesh [%ls], LOD%d section %d doesn't exist.",
				*Asset.GetName(),
				*SkeletalMesh.GetName(),
				MeshLodIndex,
				LodSectionIndex);
			return;
		}

		for (int32 SectionIndex = EndLodSectionIndex; SectionIndex >= StartLodSectionIndex; --SectionIndex)
		{
			FSkelMeshSection& Section = LodModel.Sections[SectionIndex];
			if (Section.HasClothingData() && Section.ClothingData.AssetGuid == Asset.GetAssetGuid())
			{
				// Don't do this when in async task, this should have been done before initiating the build
				if (IsInGameThread() && !bChangedMesh)
				{
					SkeletalMesh.PreEditChange(nullptr);
				}

				// Log the LOD bias mapping data that will be removed
				for (int32 BiasLodIndex = 0; BiasLodIndex < LodIndex; ++BiasLodIndex)
				{
					const int32 LodBias = LodIndex - BiasLodIndex;

					UE_CLOGF(Section.ClothMappingDataLODs.IsValidIndex(LodBias) && Section.ClothMappingDataLODs[LodBias].Num(),
						LogClothingAsset, Verbose,
						"Removed deformer data for [%ls/%ls], section %d LOD bias = %d, render mesh LOD = %d, sim mesh LOD = %d",
						*SkeletalMesh.GetName(),
						*Asset.GetName(),
						SectionIndex,
						LodBias,
						LodIndex,
						BiasLodIndex);
				}

				Section.ClothingData.AssetGuid = FGuid();
				Section.ClothingData.AssetLodIndex = INDEX_NONE;
				Section.CorrespondClothAssetIndex = INDEX_NONE;
				Section.ClothMappingDataLODs.Empty();

				if (FSkelMeshSourceSectionUserData* const UserSectionData = LodModel.UserSectionsData.Find(Section.OriginalDataSectionIndex))
				{
					UserSectionData->CorrespondClothAssetIndex = INDEX_NONE;
					UserSectionData->ClothingData.AssetLodIndex = INDEX_NONE;
					UserSectionData->ClothingData.AssetGuid = FGuid();
				}

				// Clear all remaining LOD bias mapping data from other LODs that relied on this section
				ClothingAssetUtils::ClearLODBiasMappings(SkeletalMesh, LodIndex, SectionIndex);

				bChangedMesh = true;
			}
		}
	}
}

bool ClothingAssetUtils::HasAnyMeshClothingAssetBindings(const USkeletalMesh* SkeletalMesh)
{
	if (const FSkeletalMeshModel* const MeshModel = SkeletalMesh ? SkeletalMesh->GetImportedModel() : nullptr)
	{
		for (int32 LodIndex = 0; LodIndex < MeshModel->LODModels.Num(); ++LodIndex)
		{
			if (MeshModel->LODModels[LodIndex].HasClothData())
			{
				return true;
			}
		}
	}
	return false;
}

bool ClothingAssetUtils::HasAnyLodMeshClothingAssetBindings(const USkeletalMesh* SkeletalMesh, int32 LodIndex)
{
	if (const FSkeletalMeshModel* const MeshModel = SkeletalMesh ? SkeletalMesh->GetImportedModel() : nullptr)
	{
		if (MeshModel->LODModels.IsValidIndex(LodIndex) && MeshModel->LODModels[LodIndex].HasClothData())
		{
			return true;
		}
	}
	return false;
}

void ClothingAssetUtils::GetAllMeshClothingAssetBindings(const USkeletalMesh* SkeletalMesh, TArray<FClothingAssetMeshBinding>& OutBindings)
{
	OutBindings.Empty();
	if (!SkeletalMesh)
	{
		return;
	}
	if (const FSkeletalMeshModel* MeshModel = SkeletalMesh->GetImportedModel())
	{
		int32 LODNum = MeshModel->LODModels.Num();
		for (int32 LODIndex = 0; LODIndex < LODNum; ++LODIndex)
		{
			const FSkeletalMeshLODModel& MeshLODModel = MeshModel->LODModels[LODIndex];
			if (MeshLODModel.HasClothData())
			{
				TArray<FClothingAssetMeshBinding> LodBindings;
				GetAllLodMeshClothingAssetBindings(SkeletalMesh, LodBindings, LODIndex);
				OutBindings.Append(LodBindings);
			}
		}
	}
}

void ClothingAssetUtils::GetAllLodMeshClothingAssetBindings(const USkeletalMesh* SkeletalMesh, TArray<FClothingAssetMeshBinding>& OutBindings, int32 InLodIndex)
{
	OutBindings.Empty();
	if (!SkeletalMesh || !SkeletalMesh->GetImportedModel())
	{
		return;
	}
	const FSkeletalMeshModel* MeshModel = SkeletalMesh->GetImportedModel();
	if (!MeshModel || !MeshModel->LODModels.IsValidIndex(InLodIndex))
	{
		return;
	}
	const FSkeletalMeshLODModel& MeshLODModel = MeshModel->LODModels[InLodIndex];

	if (MeshLODModel.HasClothData())
	{
		TArray<FClothingAssetMeshBinding> LodBindings;
		int32 SectionNum = MeshLODModel.Sections.Num();
		for (int32 SectionIndex = 0; SectionIndex < SectionNum; ++SectionIndex)
		{
			const FSkelMeshSection& Section = MeshLODModel.Sections[SectionIndex];
			if (Section.HasClothingData())
			{
				FClothingAssetMeshBinding ClothBinding;
				ClothBinding.Asset = SkeletalMesh->GetClothingAsset(Section.ClothingData.AssetGuid);
				ClothBinding.AssetInternalLodIndex = Section.ClothingData.AssetLodIndex;
				check(ClothBinding.AssetInternalLodIndex == Section.ClothingData.AssetLodIndex);
				ClothBinding.LODIndex = InLodIndex;
				ClothBinding.SectionIndex = SectionIndex;
				OutBindings.Add(ClothBinding);
			}
		}
	}
}

void ClothingAssetUtils::UpdateLODBiasMappings(
	const UClothingAssetBase& Asset,
	const USkeletalMesh* SkeletalMesh,
	const int32 UpdatedLODIndex,
	const int32 SectionIndex,
	const bool bSmoothTransition,
	const bool bUseMultipleInfluences,
	const float SkinningKernelRadius)
{
	check(SkeletalMesh);
	check(UpdatedLODIndex >= 0);

	FSkeletalMeshModel* const SkeletalMeshModel = SkeletalMesh->GetImportedModel();
	check(SkeletalMeshModel);

	const bool bAddMappingsToAnyLOD = (SkeletalMesh->GetClothLODBiasMode() == EClothLODBiasMode::MappingsToAnyLOD);
	const bool bAddMappingsToMinLOD = (SkeletalMesh->GetClothLODBiasMode() == EClothLODBiasMode::MappingsToMinLOD && SkeletalMesh->GetRayTracingMinLOD() > 0);

	TIndirectArray<FSkeletalMeshLODModel>& LODModels = SkeletalMeshModel->LODModels;

	// Render mesh desc helper lambda
	TArray<FVector3f> RenderPositions;
	TArray<FVector3f> RenderNormals;
	TArray<FVector3f> RenderTangents;
	TArray<uint32> RenderIndices;

	auto PrepareDeformerTargetDescriptor = [&LODModels, &RenderPositions, &RenderNormals, &RenderTangents, &RenderIndices](
			int32 LODIndex,
			int32 SectionIndex) -> ClothingMeshUtils::ClothMeshDesc
		{
			FSkeletalMeshLODModel& LODModel = LODModels[LODIndex];
			FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

			RenderPositions.Reset(Section.SoftVertices.Num());
			RenderNormals.Reset(Section.SoftVertices.Num());
			RenderTangents.Reset(Section.SoftVertices.Num());

			for (const FSoftSkinVertex& SoftSkinVertex : Section.SoftVertices)
			{
				RenderPositions.Add(SoftSkinVertex.Position);
				RenderNormals.Add(SoftSkinVertex.TangentZ);
				RenderTangents.Add(SoftSkinVertex.TangentX);
			}

			const TConstArrayView<uint32> Indices = TConstArrayView<uint32>(LODModel.IndexBuffer).Slice(Section.BaseIndex, Section.NumTriangles * 3);

			RenderIndices.Reset(Section.NumTriangles * 3);
			for (uint32 VertexIndex : Indices)
			{
				RenderIndices.Add(VertexIndex - Section.BaseVertexIndex);
			}

			return ClothingMeshUtils::ClothMeshDesc(RenderPositions, RenderNormals, RenderTangents, RenderIndices);
		};

	// Cloth sim mesh and max distance helper lambda
	TArray<FVector3f> SimPositions;
	TArray<uint32> SimIndices;
	FPointWeightMap SimMaxDistances;

	auto PrepareDeformerSourceDescriptor = [&SimPositions, &SimIndices, &SimMaxDistances](
			const UClothingAssetBase& ClothingAsset,
			const int32 LODIndex,
			FPointWeightMap*& OutMaxDistances) -> ClothingMeshUtils::ClothMeshDesc
		{
			TArray<float> TempMaxDistances;
			ClothingAsset.GetSimulationMesh(LODIndex, SimPositions, SimIndices, TempMaxDistances);
			if (TempMaxDistances.Num() == SimPositions.Num())
			{
				SimMaxDistances.Values = MoveTemp(TempMaxDistances);
				OutMaxDistances = &SimMaxDistances;
			}
			else
			{
				OutMaxDistances = nullptr;  // Null max distance will safely disable smoothing transition calculations
			}
			return ClothingMeshUtils::ClothMeshDesc(SimPositions, SimIndices);  // Calculate averaged normals
		};

	// Get the min and max LODs that might be affected by LOD bias and update other sections' bias with this LOD section
	int32 MinLOD = 0;
	int32 MaxLOD = 0;

	if (bAddMappingsToAnyLOD)
	{
		MinLOD = UpdatedLODIndex + 1;
		MaxLOD = LODModels.Num() - 1;
	}
	else
	{
		const int32 RayTracingMinLOD = SkeletalMesh->GetRayTracingMinLOD();
		if (bAddMappingsToMinLOD && LODModels.IsValidIndex(RayTracingMinLOD))
		{
			MinLOD = MaxLOD = RayTracingMinLOD;
		}
	}
	if (UpdatedLODIndex < MinLOD && MinLOD <= MaxLOD)
	{
		// Prepare the deformer source (simulation) mesh descriptor
		FPointWeightMap* MaxDistances;
		const ClothingMeshUtils::ClothMeshDesc SourceMesh = PrepareDeformerSourceDescriptor(Asset, UpdatedLODIndex, MaxDistances);

		// Generate the mappings for this section at the various LODs using the current simulation mesh
		// so that higher LODs have something to map to when raytracing bias is active and a higher LOD is rendered
		for (int32 LODIndex = MinLOD; LODIndex <= MaxLOD; ++LODIndex)
		{
			FSkeletalMeshLODModel& LODModel = LODModels[LODIndex];  // Note that the section LOD being updated here is the biased (raytraced) section LOD, not the UpdatedLODIndex which is done in the second loop below

			if (!LODModel.Sections.IsValidIndex(SectionIndex))
			{
				continue;
			}

			FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

			if (!Section.HasClothingData())  // Note it doesn't matter if this section has a different asset at this point, only that the shader will want to access the clothing data
			{
				continue;  // This LOD section won't render clothing and therefore can be skipped as it won't ever need mappings
			}

			// Update bias mapping to the newly updated LOD index
			const int32 LODBias = LODIndex - UpdatedLODIndex;
			check(LODBias > 0);

			Section.ClothMappingDataLODs.SetNum(FMath::Max(LODBias + 1, Section.ClothMappingDataLODs.Num()));

			TArray<FMeshToMeshVertData>& MeshToMeshData = Section.ClothMappingDataLODs[LODBias];
			MeshToMeshData.Reset();

			// Prepare the deformer target (render) mesh descriptor
			const ClothingMeshUtils::ClothMeshDesc TargetMesh = PrepareDeformerTargetDescriptor(LODIndex, SectionIndex);

			// Generate the missing bias mapping data
			ClothingMeshUtils::GenerateMeshToMeshVertData(
				MeshToMeshData,
				TargetMesh,
				SourceMesh,
				MaxDistances,
				bSmoothTransition,
				bUseMultipleInfluences,
				SkinningKernelRadius);

			UE_LOGF(LogClothingAsset, Verbose, "Added deformer data for [%ls/%ls], section %d, LODBias = %d, render mesh LOD = %d, sim mesh LOD = %d",
				*SkeletalMesh->GetName(),
				*Asset.GetName(),
				SectionIndex,
				LODBias,
				LODIndex,
				UpdatedLODIndex);
		}
	}

	// Update this LOD section bias mappings that didn't get setup since there weren't no cloth attached to it yet
	if (bAddMappingsToAnyLOD || (bAddMappingsToMinLOD && UpdatedLODIndex == SkeletalMesh->GetRayTracingMinLOD()))
	{
		FSkeletalMeshLODModel& LODModel = LODModels[UpdatedLODIndex];  // Note that the section LOD being updated here is the UpdatedLODIndex
		FSkelMeshSection& Section = LODModel.Sections[SectionIndex];   // not the biased (raytraced) section LOD which is done in the first loop above
		check(Section.HasClothingData());

		// Prepare the deformer target (render) mesh descriptor
		const ClothingMeshUtils::ClothMeshDesc TargetMesh = PrepareDeformerTargetDescriptor(UpdatedLODIndex, SectionIndex);

		// Iterate through all the LOD that could be rendered and need this source LOD to be raytraced
		for (int32 LODIndex = 0; LODIndex < UpdatedLODIndex; ++LODIndex)
		{
			FSkeletalMeshLODModel& SourceLODModel = LODModels[LODIndex];

			if (!SourceLODModel.Sections.IsValidIndex(SectionIndex))
			{
				continue;
			}

			FSkelMeshSection& SourceSection = SourceLODModel.Sections[SectionIndex];

			// Locate the deformer's source (simulation) cloth asset, since it could be a different cloth asset.
			const UClothingAssetBase* const ClothingAsset = SkeletalMesh->GetClothingAsset(SourceSection.ClothingData.AssetGuid);
			if (!ClothingAsset)
			{
				UE_LOGF(LogClothingAsset, Warning, "Incomplete raytracing cloth LOD bias mappings generation on [%ls/%ls], section %d, LOD %d due to no clothing data set to LOD %d.",
					*SkeletalMesh->GetName(),
					*Asset.GetName(),
					SectionIndex,
					UpdatedLODIndex,
					LODIndex);
				continue;
			}

			// Prepare the deformer source (simulation) mesh descriptor
			FPointWeightMap* MaxDistances;
			const ClothingMeshUtils::ClothMeshDesc SourceMesh = PrepareDeformerSourceDescriptor(*ClothingAsset, LODIndex, MaxDistances);

			// Update bias mapping for the updated LOD section
			const int32 LODBias = UpdatedLODIndex - LODIndex;
			check(LODBias > 0);

			Section.ClothMappingDataLODs.SetNum(FMath::Max(LODBias + 1, Section.ClothMappingDataLODs.Num()));
			TArray<FMeshToMeshVertData>& MeshToMeshData = Section.ClothMappingDataLODs[LODBias];
			MeshToMeshData.Reset();

			ClothingMeshUtils::GenerateMeshToMeshVertData(
				MeshToMeshData,
				TargetMesh,
				SourceMesh,
				MaxDistances,
				bSmoothTransition,
				bUseMultipleInfluences,
				SkinningKernelRadius);

			UE_LOGF(LogClothingAsset, Verbose, "Added deformer data for [%ls/%ls], section %d, LODBias = %d, render mesh LOD = %d, sim mesh LOD = %d",
				*SkeletalMesh->GetName(),
				*Asset.GetName(),
				SectionIndex,
				LODBias,
				UpdatedLODIndex,
				LODIndex);
		}
	}
}

// UE_DEPRECATED(5.7, "Unused low level function, will be removed or re-added as a member to FSkelMeshSection.")
void ClothingAssetUtils::ClearSectionClothingData(FSkelMeshSection& Section)
{
	Section.ClothingData.AssetGuid = FGuid();
	Section.ClothingData.AssetLodIndex = INDEX_NONE;
	Section.CorrespondClothAssetIndex = INDEX_NONE;

	Section.ClothMappingDataLODs.Empty();
}

void ClothingAssetUtils::ClearLODBiasMappings(USkeletalMesh& SkeletalMesh, const int32 UpdatedLodIndex, const int32 SectionIndex)
{
	if (FSkeletalMeshModel* const SkeletalMeshModel = SkeletalMesh.GetImportedModel())
	{
		if (SkeletalMeshModel->LODModels.IsValidIndex(UpdatedLodIndex))
		{
			for (int32 LODIndex = UpdatedLodIndex + 1; LODIndex < SkeletalMeshModel->LODModels.Num(); ++LODIndex)
			{
				FSkeletalMeshLODModel& BiasLodModel = SkeletalMeshModel->LODModels[LODIndex];
				if (BiasLodModel.Sections.IsValidIndex(SectionIndex))
				{
					FSkelMeshSection& BiasSection = BiasLodModel.Sections[SectionIndex];
					const int32 LODBias = LODIndex - UpdatedLodIndex;

					if (BiasSection.ClothMappingDataLODs.IsValidIndex(LODBias))
					{
						BiasSection.ClothMappingDataLODs[LODBias].Empty();

						const UClothingAssetBase* const Asset = SkeletalMesh.GetClothingAsset(BiasSection.ClothingData.AssetGuid);

						UE_LOGF(LogClothingAsset, Verbose, "Removed deformer data for [%ls/%ls], section %d LODBias = %d, render mesh LOD = %d, sim mesh LOD = %d",
							*SkeletalMesh.GetName(),
							Asset ? *Asset->GetName() : TEXT("?"),
							SectionIndex,
							LODBias,
							LODIndex,
							UpdatedLodIndex);
					}
				}
			}
		}
	}
}
#endif

//==============================================================================
// UClothingAssetCommon
//==============================================================================

UClothingAssetCommon::UClothingAssetCommon(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PhysicsAsset(nullptr)
#if WITH_EDITORONLY_DATA
	, ClothSimConfig_DEPRECATED(nullptr)
	, ChaosClothSimConfig_DEPRECATED(nullptr)
#endif
	, ReferenceBoneIndex(0)
{
}

#if WITH_EDITOR
bool UClothingAssetCommon::BindToSkeletalMesh(
	USkeletalMesh* InSkelMesh, 
	const int32 InMeshLodIndex, 
	const int32 InSectionIndex, 
	const int32 InAssetLodIndex)
{
	// Make sure the legacy LOD are upgraded (BindToSkeletalMesh could be called before the Cloth Asset's PostLoad is completed)
	for (UClothLODDataCommon_Legacy* LodDeprecated : ClothLodData_DEPRECATED)
	{
		if (LodDeprecated)
		{
			LodDeprecated->ConditionalPostLoad();

			const int32 Idx = AddNewLod();
			LodDeprecated->MigrateTo(LodData[Idx]);
		}
	}
	ClothLodData_DEPRECATED.Empty();

	// If we've been added to the wrong mesh
	if(InSkelMesh != GetOuter())
	{
		const FText Error = FText::Format(
			LOCTEXT("Error_WrongMesh", "Failed to bind clothing asset {0} as the provided mesh is not the owner of this asset."), 
			FText::FromString(GetName()));
		WarningNotification(Error);
		return false;
	}

	// If we don't have clothing data
	if(!LodData.IsValidIndex(InAssetLodIndex))
	{
		const FText Error = FText::Format(
			LOCTEXT("Error_NoClothingLod", "Failed to bind clothing asset {0} LOD{1} as LOD{2} does not exist."), 
			FText::FromString(GetName()), 
			InAssetLodIndex,
			InAssetLodIndex);
		WarningNotification(Error);
		return false;
	}

	// If we don't have a mesh
	if(!InSkelMesh)
	{
		const FText Error = FText::Format(
			LOCTEXT("Error_NoMesh", "Failed to bind clothing asset {0} as provided skel mesh does not exist."), 
			FText::FromString(GetName()));
		WarningNotification(Error);
		return false;
	}

	// If the mesh LOD index is invalid
	if(!InSkelMesh->GetImportedModel()->LODModels.IsValidIndex(InMeshLodIndex))
	{
		const FText Error = FText::Format(
			LOCTEXT("Error_InvalidMeshLOD", "Failed to bind clothing asset {0} as mesh LOD{1} does not exist."), 
			FText::FromString(GetName()), 
			InMeshLodIndex);
		WarningNotification(Error);
		return false;
	}

	const int32 NumMapEntries = LodMap.Num();
	for(int MapIndex = 0; MapIndex < NumMapEntries; ++MapIndex)
	{
		const int32& MappedLod = LodMap[MapIndex];
		if(MappedLod == InAssetLodIndex)
		{
			const FText Error = FText::Format(
				LOCTEXT("Error_LodMapped", "Failed to bind clothing asset {0} LOD{1} as LOD{2} is already mapped to mesh LOD{3}."), 
				FText::FromString(GetName()), 
				InAssetLodIndex, 
				InAssetLodIndex, 
				MapIndex);
			WarningNotification(Error);
			return false;
		}
	}

	if(LodMap.IsValidIndex(InMeshLodIndex) && LodMap[InMeshLodIndex] != INDEX_NONE)
	{
		// Already mapped
		return false;
	}

	CalculateReferenceBoneIndex();

	// Grab the clothing and skel lod data
	FClothLODDataCommon& ClothLodData = LodData[InAssetLodIndex];
	FSkeletalMeshLODModel& SkelLod = InSkelMesh->GetImportedModel()->LODModels[InMeshLodIndex];

	FSkelMeshSection& OriginalSection = SkelLod.Sections[InSectionIndex];

	// Data for mesh to mesh binding
	TArray<FMeshToMeshVertData> MeshToMeshData;
	TArray<FVector3f> RenderPositions;
	TArray<FVector3f> RenderNormals;
	TArray<FVector3f> RenderTangents;

	RenderPositions.Reserve(OriginalSection.SoftVertices.Num());
	RenderNormals.Reserve(OriginalSection.SoftVertices.Num());
	RenderTangents.Reserve(OriginalSection.SoftVertices.Num());

	// Original data to weight to the clothing simulation mesh
	for(FSoftSkinVertex& UnrealVert : OriginalSection.SoftVertices)
	{
		RenderPositions.Add(UnrealVert.Position);
		RenderNormals.Add(UnrealVert.TangentZ);
		RenderTangents.Add(UnrealVert.TangentX);
	}

	TArrayView<uint32> IndexView(SkelLod.IndexBuffer);
	IndexView = IndexView.Slice(OriginalSection.BaseIndex, OriginalSection.NumTriangles * 3);

	TArray<uint32> RenderIndices;
	RenderIndices.Reserve(OriginalSection.NumTriangles * 3);
	for (uint32 OriginalIndex : IndexView)
	{
		const int32 TempIndex = (int32)OriginalIndex - (int32)OriginalSection.BaseVertexIndex;
		if (ensure(RenderPositions.IsValidIndex(TempIndex)))
		{
			RenderIndices.Add(TempIndex);
		}
	}

	const ClothingMeshUtils::ClothMeshDesc TargetMesh(RenderPositions, RenderNormals, RenderTangents, RenderIndices);
	const ClothingMeshUtils::ClothMeshDesc SourceMesh(ClothLodData.PhysicalMeshData.Vertices, ClothLodData.PhysicalMeshData.Indices);  // Calculate averaged normals

	const FPointWeightMap* const MaxDistances = ClothLodData.PhysicalMeshData.FindWeightMap(EWeightMapTargetCommon::MaxDistance);

	ClothingMeshUtils::GenerateMeshToMeshVertData(MeshToMeshData, TargetMesh, SourceMesh, MaxDistances,
		ClothLodData.bSmoothTransition, ClothLodData.bUseMultipleInfluences, ClothLodData.SkinningKernelRadius);

	// We have to copy the bone map to verify we don't exceed the maximum while adding the clothing bones
	TArray<FBoneIndexType> TempBoneMap = OriginalSection.BoneMap;
	for(FName& BoneName : UsedBoneNames)
	{
		const int32 BoneIndex = InSkelMesh->GetRefSkeleton().FindBoneIndex(BoneName);
		if(BoneIndex != INDEX_NONE)
		{
			TempBoneMap.AddUnique(BoneIndex);
		}
	}
	
	// Copy the new bone map to the section
	OriginalSection.BoneMap = TempBoneMap;

	//Register the scope post edit change
	FScopedSkeletalMeshPostEditChange SkeletalMeshPostEditChange(InSkelMesh);

	// calculate LOD verts before adding our new section
	uint32 NumLodVertices = 0;
	for(const FSkelMeshSection& CurSection : SkelLod.Sections)
	{
		NumLodVertices += CurSection.GetNumVertices();
	}

	// Set the asset index, used during rendering to pick the correct sim mesh buffer
	int32 AssetIndex = INDEX_NONE;
	const bool bMeshClothingAssetsWasFound = InSkelMesh->GetMeshClothingAssets().Find(this, AssetIndex);
	check(bMeshClothingAssetsWasFound);
	OriginalSection.CorrespondClothAssetIndex = AssetIndex;

	// sim properties
	OriginalSection.ClothMappingDataLODs.SetNum(1);
	OriginalSection.ClothMappingDataLODs[0] = MeshToMeshData;
	OriginalSection.ClothingData.AssetGuid = AssetGuid;
	OriginalSection.ClothingData.AssetLodIndex = InAssetLodIndex;

	bool bRequireBoneChange = false;
	for(FBoneIndexType& BoneIndex : OriginalSection.BoneMap)
	{
		if(!SkelLod.RequiredBones.Contains(BoneIndex))
		{
			bRequireBoneChange = true;
			if(InSkelMesh->GetRefSkeleton().IsValidIndex(BoneIndex))
			{
				SkelLod.RequiredBones.Add(BoneIndex);
				SkelLod.ActiveBoneIndices.AddUnique(BoneIndex);
			}
		}
	}
	if(bRequireBoneChange)
	{
		SkelLod.RequiredBones.Sort();
		InSkelMesh->GetRefSkeleton().EnsureParentsExistAndSort(SkelLod.ActiveBoneIndices);
	}

	// Make sure the LOD map is always big enough for the asset to use.
	// This shouldn't grow to an unwieldy size but maybe consider compacting later.
	while(LodMap.Num() - 1 < InMeshLodIndex)
	{
		LodMap.Add(INDEX_NONE);
	}

	LodMap[InMeshLodIndex] = InAssetLodIndex;

	// Update the extra cloth deformer mapping LOD bias using this cloth entry
	if (InSkelMesh->GetSupportRayTracing())
	{
		check(LodMap[InMeshLodIndex] == InAssetLodIndex);  // UpdateLODBiasMappings relies on the LodMap being up to date
		ClothingAssetUtils::UpdateLODBiasMappings(
			*this,
			InSkelMesh,
			InMeshLodIndex,
			InSectionIndex,
			ClothLodData.bSmoothTransition,
			ClothLodData.bUseMultipleInfluences,
			ClothLodData.SkinningKernelRadius);
	}

	return true;

	// FScopedSkeletalMeshPostEditChange goes out of scope, causing postedit change and components to be re-registered
}

void UClothingAssetCommon::UpdateAllLODBiasMappings(USkeletalMesh* SkeletalMesh)
{
	check(SkeletalMesh);

	FSkeletalMeshModel* const SkeletalMeshModel = SkeletalMesh->GetImportedModel();
	if (!SkeletalMeshModel)
	{
		return;
	}

	// Iterate through all source LODs with cloth that could lead to upper (raytraced) sections needing some additional mapping
	TIndirectArray<FSkeletalMeshLODModel>& LODModels = SkeletalMeshModel->LODModels;

	for (int32 LODIndex = LODModels.Num() - 1; LODIndex >= 0; --LODIndex)  // Iterate in reverse order to allow shrinking the mapping array first
	{
		if (LodMap.IsValidIndex(LODIndex) && LodData.IsValidIndex(LodMap[LODIndex]))
		{
			const FClothLODDataCommon& ClothLodData = LodData[LodMap[LODIndex]];

			// Go through all sections to find the one(s?) that uses this cloth asset and clear the existing bias mappings
			TArray<FSkelMeshSection>& Sections = LODModels[LODIndex].Sections;
		
			for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
			{
				if (Sections[SectionIndex].ClothingData.AssetGuid == GetAssetGuid())
				{
					Sections[SectionIndex].ClothMappingDataLODs.SetNum(1);  // Only keep ClothMappingDataLODs[0] that is the same LOD mapping to remove LOD bias mappings

					if (SkeletalMesh->GetSupportRayTracing())  // This conditional is inside the loop so that the SetNum(1) can removed the unused bias mappings
					{
						// Updates the upper LODs mappings of the specified section from this LODIndex
						ClothingAssetUtils::UpdateLODBiasMappings(
							*this,
							SkeletalMesh,
							LODIndex,
							SectionIndex,
							ClothLodData.bSmoothTransition,
							ClothLodData.bUseMultipleInfluences,
							ClothLodData.SkinningKernelRadius);
					}
				}
			}
		}
	}
}

void UClothingAssetCommon::GetSimulationMesh(const int32 InMeshLodIndex, TArray<FVector3f>& OutPositions, TArray<uint32>& OutIndices, TArray<float>& OutMaxDistances)const
{
	OutPositions.Reset();
	OutIndices.Reset();
	OutMaxDistances.Reset();

	if (LodMap.IsValidIndex(InMeshLodIndex) && LodData.IsValidIndex(LodMap[InMeshLodIndex]))
	{
		const FClothPhysicalMeshData& PhysicalMeshData = LodData[LodMap[InMeshLodIndex]].PhysicalMeshData;

		OutPositions = PhysicalMeshData.Vertices;
		OutIndices = PhysicalMeshData.Indices;
		if (const FPointWeightMap* const MaxDistances = PhysicalMeshData.FindWeightMap(EWeightMapTargetCommon::MaxDistance);
			MaxDistances && MaxDistances->Num() == OutPositions.Num())
		{
			OutMaxDistances = MaxDistances->Values;
		}
	}
}

void UClothingAssetCommon::UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex)
{
	// Clear the LOD map entries to allow for rebinding a new asset
	if (InMeshLodIndex == INDEX_NONE)
	{
		for (int32& LodMapIndex : LodMap)
		{
			LodMapIndex = INDEX_NONE;  // Note: This must be done prior calling UnbindFromSkeletalMesh(), so that the info is correct when rebuilding the mesh
		}
	}
	else if (LodMap.IsValidIndex(InMeshLodIndex))
	{
		LodMap[InMeshLodIndex] = INDEX_NONE;  // Note: This must be done prior calling UnbindFromSkeletalMesh(), so that the info is correct when rebuilding the mesh
	}
	// Unbind from the Skeletal Mesh
	if (InSkelMesh)
	{
		ClothingAssetUtils::UnbindFromSkeletalMesh(*this, *InSkelMesh, InMeshLodIndex, InSectionIndex);
	}
}

void UClothingAssetCommon::ReregisterComponentsUsingClothing()
{
	if(USkeletalMesh* OwnerMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		for(TObjectIterator<USkeletalMeshComponent> It; It; ++It)
		{
			if(USkeletalMeshComponent* Component = *It)
			{
				if(Component->GetSkeletalMeshAsset() == OwnerMesh)
				{
					FComponentReregisterContext Context(Component);
					// Context goes out of scope, causing Component to be re-registered
				}
			}
		}
	}
}

void UClothingAssetCommon::ForEachInteractorUsingClothing(TFunction<void(UClothingSimulationInteractor*)> Func)
{
	if(USkeletalMesh* OwnerMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		for(TObjectIterator<USkeletalMeshComponent> It; It; ++It)
		{
			if(USkeletalMeshComponent* Component = *It)
			{
				if(Component->GetSkeletalMeshAsset() == OwnerMesh)
				{
					if(UClothingSimulationInteractor* CurInteractor = Component->GetClothingSimulationInteractor(GetFName()))
					{
						Func(CurInteractor);
					}
				}
			}
		}
	}
}

void UClothingAssetCommon::ApplyParameterMasks(bool bUpdateFixedVertData, bool bInvalidateDerivedDataCache)
{
	for(FClothLODDataCommon& Lod : LodData)
	{
		Lod.PushWeightsToMesh();
	}
	// Invalidate all cached data that depend on masks
	InvalidateFlaggedCachedData(EClothingCachedDataFlagsCommon::InverseMasses | EClothingCachedDataFlagsCommon::Tethers);

	// Recompute weights if needed
	USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(GetOuter());
	
	if (bUpdateFixedVertData && SkeletalMesh)
	{
		if (FSkeletalMeshModel* const Resource = SkeletalMesh->GetImportedModel())
		{
			FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(SkeletalMesh);

			SkeletalMesh->PreEditChange(nullptr);

			for (FSkeletalMeshLODModel& LodModel : Resource->LODModels)
			{
				for (FSkelMeshSection& Section : LodModel.Sections)
				{
					if (!Section.HasClothingData() || Cast<UClothingAssetCommon>(SkeletalMesh->GetClothingAsset(Section.ClothingData.AssetGuid)) != this)
					{
						continue;
					}
					const FClothLODDataCommon& LodDatum = LodData[Section.ClothingData.AssetLodIndex];
					const FPointWeightMap* const MaxDistances = LodDatum.PhysicalMeshData.FindWeightMap(EWeightMapTargetCommon::MaxDistance);

					for (TArray<FMeshToMeshVertData>& ClothMappingData : Section.ClothMappingDataLODs)
					{
						ClothingMeshUtils::ComputeVertexContributions(ClothMappingData, MaxDistances, LodDatum.bSmoothTransition, LodDatum.bUseMultipleInfluences);
					}
				}
			}
			if (bInvalidateDerivedDataCache)  // We must always dirty the DDC key unless previewing
			{
				SkeletalMesh->InvalidateDeriveDataCacheGUID();
			}
		}
	}
}

void UClothingAssetCommon::BuildLodTransitionData()
{
	const int32 NumLods = GetNumLods();
	for(int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
	{
		const bool bHasPrevLod = LodIndex > 0;
		const bool bHasNextLod = LodIndex < NumLods - 1;

		FClothLODDataCommon& CurrentLod = LodData[LodIndex];
		const FClothPhysicalMeshData& CurrentPhysMesh = CurrentLod.PhysicalMeshData;

		FClothLODDataCommon* const PrevLod = bHasPrevLod ? &LodData[LodIndex - 1] : nullptr;
		FClothLODDataCommon* const NextLod = bHasNextLod ? &LodData[LodIndex + 1] : nullptr;

		const int32 CurrentLodNumVerts = CurrentPhysMesh.Vertices.Num();

		ClothingMeshUtils::ClothMeshDesc CurrentMeshDesc(CurrentPhysMesh.Vertices, CurrentPhysMesh.Normals, CurrentPhysMesh.Indices);

		const FPointWeightMap* MaxDistances = nullptr; // No need to update the vertex contribution on the transition maps
		constexpr bool bUseSmoothTransitions = false;  // Smooth transitions are only used at rendering for now and not during LOD transitions
		constexpr bool bUseMultipleInfluences = false;  // Multiple influences must not be used for LOD transitions
		constexpr float SkinningKernelRadius = 0.f;  // KernelRadius is only required when using multiple influences

		if(PrevLod)
		{
			FClothPhysicalMeshData& PrevPhysMesh = PrevLod->PhysicalMeshData;
			CurrentLod.TransitionUpSkinData.Empty(CurrentLodNumVerts);
			ClothingMeshUtils::ClothMeshDesc PrevMeshDesc(PrevPhysMesh.Vertices, PrevPhysMesh.Indices);  // Will calculate averaged normals

			ClothingMeshUtils::GenerateMeshToMeshVertData(CurrentLod.TransitionUpSkinData, CurrentMeshDesc, PrevMeshDesc,
				MaxDistances, bUseSmoothTransitions, bUseMultipleInfluences, SkinningKernelRadius);
		}
		if(NextLod)
		{
			FClothPhysicalMeshData& NextPhysMesh = NextLod->PhysicalMeshData;
			CurrentLod.TransitionDownSkinData.Empty(CurrentLodNumVerts);
			ClothingMeshUtils::ClothMeshDesc NextMeshDesc(NextPhysMesh.Vertices, NextPhysMesh.Indices);  // Will calculate averaged normals 
			ClothingMeshUtils::GenerateMeshToMeshVertData(CurrentLod.TransitionDownSkinData, CurrentMeshDesc, NextMeshDesc,
				MaxDistances, bUseSmoothTransitions, bUseMultipleInfluences, SkinningKernelRadius);
		}
	}
}

void UClothingAssetCommon::PreEditUndo()
{
	Super::PreEditUndo();

	// Stop the simulation
	if (const USkeletalMesh* const OwnerMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
		{
			if (USkeletalMeshComponent* const Component = *It)
			{
				if (Component->GetSkeletalMeshAsset() == OwnerMesh)
				{
					Component->ReleaseAllClothingResources();
				}
			}
		}
	}
}

void UClothingAssetCommon::PostEditUndo()
{
	Super::PostEditUndo();

	// Resume the simulation
	if (const USkeletalMesh* const OwnerMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
		{
			if (USkeletalMeshComponent* const Component = *It)
			{
				if (Component->GetSkeletalMeshAsset() == OwnerMesh)
				{
					Component->RecreateClothingActors();
				}
			}
		}
	}
}

TArray<TPair<FText, FText>> UClothingAssetCommon::GetStats() const
{
	TArray<TPair<FText, FText>> Stats;

	for (int32 LodIndex = 0; LodIndex < LodData.Num(); ++LodIndex)
	{
		Stats.Emplace(FText::Format(LOCTEXT("LodIndexLabel", "LOD {0}"), FText::AsNumber(LodIndex)), FText{});
		const FClothPhysicalMeshData& PhysMeshData = LodData[LodIndex].PhysicalMeshData;
		Stats.Emplace(LOCTEXT("SimulVertexCount", "Simul Verts"), FText::AsNumber(PhysMeshData.Vertices.Num() - PhysMeshData.NumFixedVerts));
		Stats.Emplace(LOCTEXT("FixedVertexCount", "Fixed Verts"), FText::AsNumber(PhysMeshData.NumFixedVerts));
		Stats.Emplace(LOCTEXT("TriangleCount", "Sim Triangles"), FText::AsNumber(PhysMeshData.Indices.Num() / 3));
		Stats.Emplace(LOCTEXT("MaxBoneWeights", "Bone Weights"), FText::AsNumber(PhysMeshData.MaxBoneWeights));
	}
	return Stats;
}
#endif // WITH_EDITOR

void UClothingAssetCommon::RefreshBoneMapping(USkeletalMesh* InSkelMesh)
{
	// No mesh, can't remap
	if(!InSkelMesh)
	{
		return;
	}

	if(UsedBoneNames.Num() != UsedBoneIndices.Num())
	{
		UsedBoneIndices.Reset();
		UsedBoneIndices.AddDefaulted(UsedBoneNames.Num());
	}

	// Repopulate the used indices.
	for(int32 BoneNameIndex = 0; BoneNameIndex < UsedBoneNames.Num(); ++BoneNameIndex)
	{
		const int32 BoneIndex = InSkelMesh->GetRefSkeleton().FindBoneIndex(UsedBoneNames[BoneNameIndex]);
		UsedBoneIndices[BoneNameIndex] = (BoneIndex != INDEX_NONE) ? BoneIndex : 0;  // Remap to the root bone if the two skeletons differ
	}
}

void UClothingAssetCommon::CalculateReferenceBoneIndex()
{
	// Starts at root
	ReferenceBoneIndex = 0;

	// Find the root bone for this clothing asset (common bone for all used bones)
	typedef TArray<int32> BoneIndexArray;

	// List of valid paths to the root bone from each weighted bone
	TArray<BoneIndexArray> PathsToRoot;
	
	const USkeletalMesh* OwnerMesh = Cast<USkeletalMesh>(GetOuter());

	if(OwnerMesh)
	{
		// First build a list per used bone for it's path to root
		TArray<int32> WeightedBones;  // List of actually weighted (not just used) bones

		for(FClothLODDataCommon& CurLod : LodData)
		{
			const FClothPhysicalMeshData& MeshData = CurLod.PhysicalMeshData;
			for(const FClothVertBoneData& VertBoneData : MeshData.BoneData)
			{
				for(int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
				{
					if(VertBoneData.BoneWeights[InfluenceIndex] > SMALL_NUMBER)
					{
						const int32 UnmappedBoneIndex = VertBoneData.BoneIndices[InfluenceIndex];
						check(UsedBoneIndices.IsValidIndex(UnmappedBoneIndex));
						WeightedBones.AddUnique(UsedBoneIndices[UnmappedBoneIndex]);
					}
					else
					{
						// Hit the last weight (they're sorted)
						break;
					}
				}
			}
		}

		const int32 NumWeightedBones = WeightedBones.Num();
		PathsToRoot.Reserve(NumWeightedBones);
		
		// Compute paths to the root bone
		const FReferenceSkeleton& RefSkel = OwnerMesh->GetRefSkeleton();
		for(int32 WeightedBoneIndex = 0; WeightedBoneIndex < NumWeightedBones; ++WeightedBoneIndex)
		{
			PathsToRoot.AddDefaulted();
			BoneIndexArray& Path = PathsToRoot.Last();
			
			int32 CurrentBone = WeightedBones[WeightedBoneIndex];
			Path.Add(CurrentBone);
			
			while(CurrentBone != 0 && CurrentBone != INDEX_NONE)
			{
				CurrentBone = RefSkel.GetParentIndex(CurrentBone);
				Path.Add(CurrentBone);
			}
		}

		// Paths are from leaf->root, we want the other way
		for(BoneIndexArray& Path : PathsToRoot)
		{
			Algo::Reverse(Path);
		}

		// Verify the last common bone in all paths as the root of the sim space
		const int32 NumPaths = PathsToRoot.Num();
		if(NumPaths > 0)
		{
			BoneIndexArray& FirstPath = PathsToRoot[0];
		
			const int32 FirstPathSize = FirstPath.Num();
			for(int32 PathEntryIndex = 0; PathEntryIndex < FirstPathSize; ++PathEntryIndex)
			{
				const int32 CurrentQueryIndex = FirstPath[PathEntryIndex];
				bool bValidRoot = true;

				for(int32 PathIndex = 1; PathIndex < NumPaths; ++PathIndex)
				{
					if(!PathsToRoot[PathIndex].Contains(CurrentQueryIndex))
					{
						bValidRoot = false;
						break;
					}
				}

				if(bValidRoot)
				{
					ReferenceBoneIndex = CurrentQueryIndex;
				}
				else
				{
					// Once we fail to find a valid root we're done.
					break;
				}
			}
		}
		else
		{
			// Just use root
			ReferenceBoneIndex = 0;
		}
	}
}

bool UClothingAssetCommon::IsValidLod(int32 InLodIndex) const
{
	return LodData.IsValidIndex(InLodIndex);
}

int32 UClothingAssetCommon::GetNumLods() const
{
	return LodData.Num();
}

void UClothingAssetCommon::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Migrate the deprecated UObject based lod class to the non-UObject lod structure to prevent PostLoad dependency issues
	// TODO: Remove all UObject PostLoad dependencies.
	//       Even with these ConditionalPostLoad calls, the UObject PostLoads' order of execution cannot be guaranteed.
	for (UClothLODDataCommon_Legacy* LodDeprecated : ClothLodData_DEPRECATED)
	{
		if (LodDeprecated)
		{
			LodDeprecated->ConditionalPostLoad();

			const int32 Idx = AddNewLod();
			LodDeprecated->MigrateTo(LodData[Idx]);
		}
	}
	ClothLodData_DEPRECATED.Empty();

	const int32 AnimPhysCustomVersion = GetLinkerCustomVersion(FAnimPhysObjectVersion::GUID);
	if (AnimPhysCustomVersion < FAnimPhysObjectVersion::AddedClothingMaskWorkflow)
	{
		// Convert current parameters to masks
		for (FClothLODDataCommon& Lod : LodData)
		{
			const FClothPhysicalMeshData& PhysMesh = Lod.PhysicalMeshData;

			// Didn't do anything previously - clear out in case there's something in there
			// so we can use it correctly now.
			Lod.PointWeightMaps.Reset(3);

			// Max distances
			const FPointWeightMap* const MaxDistances = PhysMesh.FindWeightMap(EWeightMapTargetCommon::MaxDistance);
			if (MaxDistances)
			{
				Lod.PointWeightMaps.AddDefaulted();
				FPointWeightMap& MaxDistanceMask = Lod.PointWeightMaps.Last();
				MaxDistanceMask.Initialize(*MaxDistances, EWeightMapTargetCommon::MaxDistance);
			}

			// Following params are only added if necessary, if we don't have any backstop
			// radii then there's no backstops.
			const FPointWeightMap* const BackstopRadiuses = PhysMesh.FindWeightMap(EWeightMapTargetCommon::BackstopRadius);
			if (BackstopRadiuses && !BackstopRadiuses->IsZeroed())
			{
				// Backstop radii
				Lod.PointWeightMaps.AddDefaulted();
				FPointWeightMap& BackstopRadiusMask = Lod.PointWeightMaps.Last();
				BackstopRadiusMask.Initialize(*BackstopRadiuses, EWeightMapTargetCommon::BackstopRadius);

				// Backstop distances
				Lod.PointWeightMaps.AddDefaulted();
				FPointWeightMap& BackstopDistanceMask = Lod.PointWeightMaps.Last();
				const FPointWeightMap& BackstopDistances = PhysMesh.GetWeightMap(EWeightMapTargetCommon::BackstopDistance);
				BackstopDistanceMask.Initialize(BackstopDistances, EWeightMapTargetCommon::BackstopDistance);
			}
		}

		// Make sure we're transactional
		SetFlags(RF_Transactional);
	}

	const int32 ClothingCustomVersion = GetLinkerCustomVersion(FClothingAssetCustomVersion::GUID);
	// Fix content imported before we kept vertex colors
	if (ClothingCustomVersion < FClothingAssetCustomVersion::AddVertexColorsToPhysicalMesh)
	{
		for (FClothLODDataCommon& Lod : LodData)
		{
			const int32 NumVerts = Lod.PhysicalMeshData.Vertices.Num(); // number of verts

			Lod.PhysicalMeshData.VertexColors.Reset();
			Lod.PhysicalMeshData.VertexColors.AddUninitialized(NumVerts);
			for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
			{
				Lod.PhysicalMeshData.VertexColors[VertIdx] = FColor::White;
			}
		}
	}

	EClothingCachedDataFlagsCommon ClothingCachedDataFlags = EClothingCachedDataFlagsCommon::None;

	if (USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		// Use the skeletal mesh version here to prevent a circular dependency between ClothingAssetCommon and Engine/SkeletalMesh
		// and to unify all cached data invalidations, while the InvalidateCachedData method is also deprecated from ClothingAssetBase.
		if (SkeletalMesh->GetLinkerCustomVersion(FSkeletalMeshCustomVersion::GUID) < FSkeletalMeshCustomVersion::CachedClothInverseMasses)
		{
			ClothingCachedDataFlags |= EClothingCachedDataFlagsCommon::All;
		}
	}

	if (AnimPhysCustomVersion < FAnimPhysObjectVersion::CacheClothMeshInfluences)
	{
		ClothingCachedDataFlags |= EClothingCachedDataFlagsCommon::NumInfluences;
	}

	// Add any missing configs for the available cloth factories, and try to migrate them from any existing one
	// TODO: Remove all UObject PostLoad dependencies.
	//       Even with these ConditionalPostLoad calls, the UObject PostLoads' order of execution cannot be guaranteed.
	for (auto& ClothConfig : ClothConfigs)
	{
		if (ClothConfig.Value)
		{
			ClothConfig.Value->ConditionalPostLoad();  // PostLoad configs before adding new ones
		}
	}

	if (ClothSimConfig_DEPRECATED)
	{
		ClothSimConfig_DEPRECATED->ConditionalPostLoad();  // PostLoad old configs before replacing them
		ClothSimConfig_DEPRECATED->Rename(nullptr, nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);  // Rename the config so that the name doesn't collide with the new config map name
	}
	if (ChaosClothSimConfig_DEPRECATED)
	{
		ChaosClothSimConfig_DEPRECATED->ConditionalPostLoad();  // PostLoad old configs before replacing them
		ChaosClothSimConfig_DEPRECATED->Rename(nullptr, nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);  // Rename the config so that the name doesn't collide with the new config map name
	}
	if (ClothSharedSimConfig_DEPRECATED)
	{
		ClothSharedSimConfig_DEPRECATED->ConditionalPostLoad();  // PostLoad old configs before replacing them
		ClothSharedSimConfig_DEPRECATED->Rename(nullptr, nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);  // Rename the config so that the name doesn't collide with the new config map name
	}

	if (AddClothConfigs())
	{
		// With a new config added best to recache everything
		ClothingCachedDataFlags |= EClothingCachedDataFlagsCommon::All;
	}

	// Migrate configs
	bool bMigrateSharedConfigToConfig = true;  // Shared config to config migration can be disabled to avoid overriding the newly migrated values

	if (ClothingCustomVersion < FClothingAssetCustomVersion::MovePropertiesToCommonBaseClasses)
	{
		// Remap legacy struct FClothConfig to new config objects
		for (auto& ClothConfig : ClothConfigs)
		{
			if (UClothConfigCommon* const ClothConfigCommon = Cast<UClothConfigCommon>(ClothConfig.Value))
			{
				ClothConfigCommon->ConditionalPostLoad();
				ClothConfigCommon->MigrateFrom(ClothConfig_DEPRECATED);
			}
		}
		bMigrateSharedConfigToConfig = false;
	}
	else
	{
		// Migrate simulation dependent config parameters to the new config map
		if (ClothSimConfig_DEPRECATED)
		{
			// Try a remap to the new config objects through the legacy structure
			if (const UClothConfigCommon* const ClothSimConfigCommon = Cast<UClothConfigCommon>(ClothSimConfig_DEPRECATED))
			{
				FClothConfig_Legacy ClothConfigLegacy;
				if (ClothSimConfigCommon->MigrateTo(ClothConfigLegacy))
				{
					for (auto& ClothConfig : ClothConfigs)
					{
						if (UClothConfigCommon* const ClothConfigCommon = Cast<UClothConfigCommon>(ClothConfig.Value))
						{
							ClothConfigCommon->ConditionalPostLoad();
							ClothConfigCommon->MigrateFrom(ClothConfigLegacy);
						}
					}
				}
			}
			// And keep the old config too
			SetClothConfig(ToRawPtr(ClothSimConfig_DEPRECATED));
			ClothSimConfig_DEPRECATED = nullptr;
			bMigrateSharedConfigToConfig = false;
		}
		if (ChaosClothSimConfig_DEPRECATED)
		{
			SetClothConfig(ToRawPtr(ChaosClothSimConfig_DEPRECATED));
			ChaosClothSimConfig_DEPRECATED = nullptr;
			bMigrateSharedConfigToConfig = false;
		}
		if (ClothSharedSimConfig_DEPRECATED)
		{
			SetClothConfig(ToRawPtr(ClothSharedSimConfig_DEPRECATED));
			ClothSharedSimConfig_DEPRECATED = nullptr;
			bMigrateSharedConfigToConfig = false;
		}
	}

	// Propagate shared configs between cloth assets
	PropagateSharedConfigs(bMigrateSharedConfigToConfig);

	// Cache tethers and reference bone index should already have been calculated
	const int32 FortniteReleaseBranchCustomObjectVersion = GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
	const int32 SpecialProjectBranchObjectVersion = GetLinkerCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID);
	if (SpecialProjectBranchObjectVersion < FUE5SpecialProjectStreamObjectVersion::ChaosClothRemoveKinematicTethers)
	{
		ClothingCachedDataFlags |= EClothingCachedDataFlagsCommon::Tethers;
	}

	if (FortniteReleaseBranchCustomObjectVersion < FFortniteReleaseBranchCustomObjectVersion::ChaosClothAddTethersToCachedData &&
		SpecialProjectBranchObjectVersion < FUE5SpecialProjectStreamObjectVersion::ChaosClothAddTethersToCachedData)
	{
		ClothingCachedDataFlags |= EClothingCachedDataFlagsCommon::Tethers;

		// ReferenceBoneIndex is only required when rebinding the cloth.
		if (USkeletalMesh* const OwnerMesh = Cast<USkeletalMesh>(GetOuter()))
		{
			RefreshBoneMapping(OwnerMesh);
			CalculateReferenceBoneIndex();
		}
	}

	// After fixing the content, we are ready to call functions that rely on it
	if (ClothingCachedDataFlags != EClothingCachedDataFlagsCommon::None)
	{
		// Rebuild data cache
		InvalidateFlaggedCachedData(ClothingCachedDataFlags);
	}

	const int32 PhysicsObjectVersion = GetLinkerCustomVersion(FPhysicsObjectVersion::GUID);
	const int32 FortniteMainBranchObjectVersion = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (PhysicsObjectVersion < FPhysicsObjectVersion::ChaosClothFixLODTransitionMaps ||
		FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::ChaosClothFixLODTransitionMaps)
	{
		BuildLodTransitionData();
	}
#endif  // #if WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
void UClothingAssetCommon::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	const TArray<IClothingSimulationFactoryClassProvider*> ClassProviders =
		IModularFeatures::Get().GetModularFeatureImplementations<IClothingSimulationFactoryClassProvider>(IClothingSimulationFactoryClassProvider::FeatureName);
	for (IClothingSimulationFactoryClassProvider* Provider : ClassProviders)
	{
		check(Provider);
		if (UClass* const ClothingSimulationFactoryClass = *TSubclassOf<class UClothingSimulationFactory>(Provider->GetClothingSimulationFactoryClass()))
		{
			const UClothingSimulationFactory* const ClothingSimulationFactory = ClothingSimulationFactoryClass->GetDefaultObject<UClothingSimulationFactory>();
			for (TSubclassOf<UClothConfigBase> ClothConfigClass : ClothingSimulationFactory->GetClothConfigClasses())
			{
				OutConstructClasses.Add(FTopLevelAssetPath(ClothConfigClass));
			}
		}
	}
}
#endif

void UClothingAssetCommon::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FAnimPhysObjectVersion::GUID);
	Ar.UsingCustomVersion(FClothingAssetCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5SpecialProjectStreamObjectVersion::GUID);
}

bool UClothingAssetCommon::AddClothConfigs()
{
	bool bNewConfigAdded = false;

	const TArray<IClothingSimulationFactoryClassProvider*> ClassProviders =
		IModularFeatures::Get().GetModularFeatureImplementations<IClothingSimulationFactoryClassProvider>(IClothingSimulationFactoryClassProvider::FeatureName);

	for (IClothingSimulationFactoryClassProvider* Provider : ClassProviders)
	{
		check(Provider);
		if (UClass* const ClothingSimulationFactoryClass = *TSubclassOf<class UClothingSimulationFactory>(Provider->GetClothingSimulationFactoryClass()))
		{
			const UClothingSimulationFactory* const ClothingSimulationFactory = ClothingSimulationFactoryClass->GetDefaultObject<UClothingSimulationFactory>();
			if (!ClothingSimulationFactory || !ClothingSimulationFactory->SupportsAsset(this))
			{
				continue;  // No need to add configs for simulations that don't support this asset
			}
			for (TSubclassOf<UClothConfigBase> ClothConfigClass : ClothingSimulationFactory->GetClothConfigClasses())
			{
				const FName ClothConfigName = ClothConfigClass->GetFName();

				TObjectPtr<UClothConfigBase>* const ClothConfigPtr = ClothConfigs.Find(ClothConfigName);
				if (!ClothConfigPtr || !*ClothConfigPtr)
				{
					// Create new config object
					checkf(!StaticFindObject(ClothConfigClass, this, *ClothConfigClass->GetName(), EFindObjectFlags::ExactClass), TEXT("Found an existing instance of ClothConfigClass in %s. Class:%s"), *GetPathNameSafe(this), *ClothConfigClass->GetName());
					UClothConfigBase* const ClothConfig = NewObject<UClothConfigBase>(this, ClothConfigClass, ClothConfigClass->GetFName(), RF_Transactional);

					// Use the legacy config struct to try find a common config as an acceptable migration source
					// This code could be removed once the legacy code is removed, although this will then prevent
					// migration from compatible config sources
					if (UClothConfigCommon* const ClothConfigCommon = Cast<UClothConfigCommon>(ClothConfig))
					{
						for (auto ClothConfigPair : ClothConfigs)
						{
							if (const UClothConfigCommon* SourceConfig = Cast<UClothConfigCommon>(ClothConfigPair.Value))
							{
								FClothConfig_Legacy LegacyConfig;
								if (SourceConfig->MigrateTo(LegacyConfig))
								{
									ClothConfigCommon->MigrateFrom(LegacyConfig);
									break;
								}
							}
						}
					}

					// Set the new config
					check(ClothConfig);
					if (ClothConfigPtr)
					{
						*ClothConfigPtr = ClothConfig;
					}
					else
					{
						ClothConfigs.Emplace(ClothConfigName, ClothConfig);
					}
					bNewConfigAdded = true;
				}
			}
		}
	}
	return bNewConfigAdded;
}

void UClothingAssetCommon::PropagateSharedConfigs(bool bMigrateSharedConfigToConfig)
{
	// Update this asset's shared config when the asset belongs to a skeletal mesh
	if (USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(GetOuter()))
	{
		const TArray<UClothingAssetBase*>& ClothingAssets = SkeletalMesh->GetMeshClothingAssets();
 
		// Collect all shared configs found in the other assets
		decltype(ClothConfigs) ClothSharedConfigs;

		for (const UClothingAssetBase* ClothingAssetBase : ClothingAssets)
		{
			if (ClothingAssetBase == static_cast<UClothingAssetBase* >(this))
			{
				continue;
			}

			// Only common assets have shared configs
			if (const UClothingAssetCommon* const ClothingAsset = Cast<UClothingAssetCommon>(ClothingAssetBase))
			{
				// Reserve space in the map, use the total number of configs in case they're unlikely all shared configs
				const int32 Max = ClothSharedConfigs.Num() + ClothingAsset->ClothConfigs.Num();
				ClothSharedConfigs.Reserve(Max);

				// Iterate through all configs, and find the shared ones
				for (const auto& ClothSharedConfigItem : ClothingAsset->ClothConfigs)
				{
					if (Cast<UClothSharedConfigCommon>(ClothSharedConfigItem.Value) &&  // Only needs shared configs
						!ClothSharedConfigs.Find(ClothSharedConfigItem.Key))            // Only needs a single shared config per type
					{
						ClothSharedConfigs.Add(ClothSharedConfigItem);
					}
				}
			}
		}

		// Propagate the found shared configs to this asset
		for (const auto& ClothSharedConfigItem : ClothSharedConfigs)
		{
			// Set share config
			if (TObjectPtr<UClothConfigBase>* const ClothConfigBase = ClothConfigs.Find(ClothSharedConfigItem.Key))
			{
				// Reset this shared config
				*ClothConfigBase = ClothSharedConfigItem.Value;
			}
			else
			{
				// Add new map entry
				ClothConfigs.Add(ClothSharedConfigItem);
			}
		}

		// Migrate the common shared configs' deprecated parameters to all per cloth configs, and fix the shared config ownership
		for (TPair<FName, TObjectPtr<UClothConfigBase>>& ClothSharedConfigItem : ClothConfigs)
		{
			if (UClothSharedConfigCommon* const ClothSharedConfig = Cast<UClothSharedConfigCommon>(ClothSharedConfigItem.Value))
			{
				// Migrate from this shared config to non shared configs if needed
				if (bMigrateSharedConfigToConfig)
				{
					for (const TPair<FName, TObjectPtr<UClothConfigBase>>& ClothConfigItem : ClothConfigs)
					{
						if (Cast<UClothSharedConfigCommon>(ClothConfigItem.Value))
						{
							continue;  // Don't migrate shared configs to another shared configs (or itself)
						}
						if (UClothConfigCommon* const ClothConfig = Cast<UClothConfigCommon>(ClothConfigItem.Value))
						{
							ClothConfig->MigrateFrom(ClothSharedConfig);
						}
					}
				}
				// Fix the shared config outer if it is still a UClothingAssetCommon (the config must belong to the skeletal mesh, as it is shared between assets)
				if (Cast<UClothingAssetCommon>(ClothSharedConfig->GetOuter()))
				{
					ClothSharedConfig->Rename(nullptr, SkeletalMesh, REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
				}

				// Fix the shared config ownership, asset might have been copied and the shared config could still point to a different skeletal mesh
				if (ClothSharedConfig->GetOuter() != SkeletalMesh)
				{
					ClothSharedConfigItem.Value = DuplicateObject<UClothSharedConfigCommon>(ClothSharedConfig, SkeletalMesh, ClothSharedConfig->GetFName());
				}
			}
		}
	}
}

void UClothingAssetCommon::PostUpdateAllAssets()
{
	// Add any missing configs for the available cloth factories, and try to migrate them from any existing one
	const bool bInvalidateCachedData = AddClothConfigs();

	// Propagate shared configs
	PropagateSharedConfigs();

#if WITH_EDITORONLY_DATA
	// Invalidate cached data if the configs have changed
	if (bInvalidateCachedData)
	{
		InvalidateAllCachedData();
	}
#endif
}


#if WITH_EDITORONLY_DATA
void UClothingAssetCommon::InvalidateFlaggedCachedData(EClothingCachedDataFlagsCommon Flags)
{
	const bool bNeedsInverseMasses = EnumHasAllFlags((EClothingCachedDataFlagsCommon)Flags, EClothingCachedDataFlagsCommon::InverseMasses) && AnyOfClothConfigs([](UClothConfigBase& Config) { return Config.NeedsInverseMasses(); });
	const bool bNeedsNumInfluences = EnumHasAllFlags((EClothingCachedDataFlagsCommon)Flags, EClothingCachedDataFlagsCommon::NumInfluences) && AnyOfClothConfigs([](UClothConfigBase& Config) { return Config.NeedsNumInfluences(); });
	const bool bNeedsSelfCollisionData = EnumHasAllFlags((EClothingCachedDataFlagsCommon)Flags, EClothingCachedDataFlagsCommon::SelfCollisionData) && AnyOfClothConfigs([](UClothConfigBase& Config) { return Config.NeedsSelfCollisionData(); });
	const bool bNeedsTethers = EnumHasAllFlags((EClothingCachedDataFlagsCommon)Flags, EClothingCachedDataFlagsCommon::Tethers) && AnyOfClothConfigs([](UClothConfigBase& Config) { return Config.NeedsTethers(); });

	float SelfCollisionRadius = 0.f;
	if (bNeedsSelfCollisionData)
	{
		// Note: Only PhysX based NvCloth needs to build the SelfCollisionIndices at the moment
		for (const TPair<FName, TObjectPtr<UClothConfigBase>>& ClothConfig : ClothConfigs)
		{
			SelfCollisionRadius = FMath::Max(SelfCollisionRadius, ClothConfig.Value->GetSelfCollisionRadius());
		}
	}

	bool bThethersUseEuclideanDistance = false;
	bool bThethersUseGeodesicDistance = false;
	if (bNeedsTethers)
	{
		for (const TPair<FName, TObjectPtr<UClothConfigBase>>& ClothConfig : ClothConfigs)
		{
			if (ClothConfig.Value->NeedsTethers())
			{
				if (ClothConfig.Value->TethersUseGeodesicDistance())
				{
					bThethersUseGeodesicDistance = true;
				}
				else
				{
					bThethersUseEuclideanDistance = true;
				}
			}
		}
	}

	// Recalculate cached data
	bool bHasClothChanged = false;
	for (FClothLODDataCommon& CurrentLodData : LodData)
	{
		FClothPhysicalMeshData& PhysMesh = CurrentLodData.PhysicalMeshData;

		if (bNeedsInverseMasses)
		{
			PhysMesh.CalculateInverseMasses();
			bHasClothChanged = true;
		}

		if (bNeedsNumInfluences)
		{
			PhysMesh.CalculateNumInfluences();
			bHasClothChanged = true;
		}

		if (bNeedsSelfCollisionData)
		{
			PhysMesh.BuildSelfCollisionData(SelfCollisionRadius);
			bHasClothChanged = true;
		}

		if (bNeedsTethers)
		{
			PhysMesh.CalculateTethers(bThethersUseEuclideanDistance, bThethersUseGeodesicDistance);
			bHasClothChanged = true;
		}
	}

	// Inform the simulations that the cloth has changed
	if (bHasClothChanged)
	{
		ForEachInteractorUsingClothing([](UClothingSimulationInteractor* InInteractor)
		{
			if (InInteractor)
			{
				InInteractor->ClothConfigUpdated();
			}
		});
	}
}
#endif // #if WITH_EDITORONLY_DATA

#if WITH_EDITOR
int32 UClothingAssetCommon::AddNewLod()
{
	return LodData.AddDefaulted();
}

void UClothingAssetCommon::RemoveLastLod()
{
	const int32 LodIndex = LodData.Num() - 1;
	if (LodIndex > 0)  // Can't remove LOD0
	{
		TOptional<FScopedSkeletalMeshPostEditChange> SkeletalMeshPostEditChange;
		USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(GetOuter());
		if (SkeletalMesh)
		{
			SkeletalMeshPostEditChange.Emplace(SkeletalMesh);
		}

		int32 MeshLodIndex;
		while (LodMap.Find(LodIndex, MeshLodIndex))
		{
			LodMap[MeshLodIndex] = INDEX_NONE;  // Note: This must be done prior calling UnbindFromSkeletalMesh(), so that the info is correct when rebuilding the mesh

			if (SkeletalMesh)
			{
				// Unbind from the Skeletal Mesh
				ClothingAssetUtils::UnbindFromSkeletalMesh(*this, *SkeletalMesh, MeshLodIndex, INDEX_NONE);
			}
		}
		LodData.RemoveAt(LodIndex, EAllowShrinking::Yes);
	}
}

void UClothingAssetCommon::PostEditChangeChainProperty(FPropertyChangedChainEvent& ChainEvent)
{
	Super::PostEditChangeChainProperty(ChainEvent);

	bool bReregisterComponents = false;

	if (ChainEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		const FName& PropertyName = ChainEvent.PropertyChain.GetActiveMemberNode() && ChainEvent.PropertyChain.GetActiveMemberNode()->GetNextNode() ?
			ChainEvent.PropertyChain.GetActiveMemberNode()->GetNextNode()->GetValue()->GetFName() : NAME_None;

		if (PropertyName == FName("bUseSelfCollisionSpheres") ||
			PropertyName == FName("SelfCollisionSphereRadius") ||
			PropertyName == FName("SelfCollisionSphereRadiusCullMultiplier"))
		{
			InvalidateFlaggedCachedData(EClothingCachedDataFlagsCommon::SelfCollisionData);
			bReregisterComponents = true;
		}
		else if (PropertyName == FName("bUseGeodesicDistance"))
		{
			InvalidateFlaggedCachedData(EClothingCachedDataFlagsCommon::Tethers);
			bReregisterComponents = true;
		}
		else if(ChainEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UClothingAssetCommon, PhysicsAsset))
		{
			bReregisterComponents = true;
		}
		else
		{
			// Other properties just require a config refresh
			ForEachInteractorUsingClothing([](UClothingSimulationInteractor* InInteractor)
			{
				if (InInteractor)
				{
					InInteractor->ClothConfigUpdated();
				}
			});
		}
	}

	if (bReregisterComponents)
	{
		ReregisterComponentsUsingClothing();
	}
}

#endif // WITH_EDITOR


#undef LOCTEXT_NAMESPACE

