// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingAssetCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FClothingAssetCustomVersion::GUID(0xFB680AF2, 0x59EF4BA3, 0xBAA819B5, 0x73C8443D);

// Register the custom version with core
FCustomVersionRegistration GRegisterClothingAssetCustomVersion(FClothingAssetCustomVersion::GUID, FClothingAssetCustomVersion::LatestVersion, TEXT("ClothingAssetVer"));
