// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for clothing assets
struct FClothingAssetCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		// Added storage of vertex colors with sim data, for editor usage
		AddVertexColorsToPhysicalMesh = 1,
		// Changed memory layouts by moving some properties to base classes
		MovePropertiesToCommonBaseClasses = 2,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FClothingAssetCustomVersion() {}
};
