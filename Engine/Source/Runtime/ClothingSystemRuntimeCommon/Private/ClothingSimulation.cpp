// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulation.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "PhysicsEngine/PhysicsSettings.h"

//==============================================================================
// FClothingSimulationContextCommon
//==============================================================================
DEFINE_STAT(STAT_ClothComputeNormals);
DEFINE_STAT(STAT_ClothInternalSolve);
DEFINE_STAT(STAT_ClothUpdateCollisions);
DEFINE_STAT(STAT_ClothSkinPhysMesh);
DEFINE_STAT(STAT_ClothFillContext);

static TAutoConsoleVariable<float> GClothMaxDeltaTimeTeleportMultiplier(
	TEXT("p.Cloth.MaxDeltaTimeTeleportMultiplier"),
	1.5f,
	TEXT("A multiplier of the MaxPhysicsDelta time at which we will automatically just teleport cloth to its new location\n")
	TEXT(" default: 1.5"));

static TAutoConsoleVariable<float> GClothMaxVelocityScale(
	TEXT("p.Cloth.MaxVelocityScale"),
	1.f,
	TEXT("The maximum amount of the component induced velocity allowed on all cloths.\n")
	TEXT("Use 1.0 for fully induced velocity(default), or use 0.0 for no induced velocity, and any other values in between for a reduced induced velocity.\n")
	TEXT("When set to 0.0, it also provides a way to force the clothing to simulate in local space.\n")
	TEXT(" default: 1.0"));

PRAGMA_DISABLE_DEPRECATION_WARNINGS // CachedPositions, CachedVelocities
FClothingSimulationContextCommon::FClothingSimulationContextCommon()
	: ComponentToWorld(FTransform::Identity)
	, WorldGravity(FVector::ZeroVector)
	, WindVelocity(FVector::ZeroVector)
	, WindAdaption(0.f)
	, DeltaSeconds(0.f)
	, VelocityScale(1.f)
	, TeleportMode(EClothingTeleportMode::None)
	, MaxDistanceScale(1.f)
	, PredictedLod(INDEX_NONE)
{}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS // CachedPositions, CachedVelocities
FClothingSimulationContextCommon::~FClothingSimulationContextCommon()
{}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FClothingSimulationContextCommon::Fill(const USkeletalMeshComponent* InComponent, float InDeltaSeconds, float InMaxPhysicsDelta, bool bIsInitialization)
{
	SCOPE_CYCLE_COUNTER(STAT_ClothFillContext);
	LLM_SCOPE(ELLMTag::SkeletalMesh);

	check(InComponent);
	FillBoneTransforms(InComponent);
	FillRefToLocals(InComponent, bIsInitialization);
	FillComponentToWorld(InComponent);
	FillWorldGravity(InComponent);
	FillWindVelocity(InComponent);
	FillDeltaSeconds(InDeltaSeconds, InMaxPhysicsDelta);
	FillTeleportMode(InComponent, InDeltaSeconds, InMaxPhysicsDelta);
	FillMaxDistanceScale(InComponent);
	FillSolverGeometryScale(InComponent);

	PredictedLod = InComponent->GetPredictedLODLevel();
}

void FClothingSimulationContextCommon::FillBoneTransforms(const USkeletalMeshComponent* InComponent)
{
	const USkeletalMesh* const SkeletalMesh = InComponent->GetSkeletalMeshAsset();

	if (USkinnedMeshComponent* const LeaderComponent = InComponent->LeaderPoseComponent.Get())
	{
		const TArray<int32>& LeaderBoneMap = InComponent->GetLeaderBoneMap();
		int32 NumBones = LeaderBoneMap.Num();

		if (NumBones == 0)
		{
			if (SkeletalMesh)
			{
				// This case indicates an invalid leader pose component (e.g. no skeletal mesh)
				NumBones = SkeletalMesh->GetRefSkeleton().GetNum();

				BoneTransforms.Empty(NumBones);
				BoneTransforms.AddDefaulted(NumBones);
			}
		}
		else
		{
			BoneTransforms.Reset(NumBones);
			BoneTransforms.AddDefaulted(NumBones);

			const TArray<FTransform>& LeaderTransforms = LeaderComponent->GetComponentSpaceTransforms();
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				bool bFoundLeader = false;
				if (LeaderBoneMap.IsValidIndex(BoneIndex))
				{
					const int32 LeaderIndex = LeaderBoneMap[BoneIndex];
					if (LeaderIndex != INDEX_NONE && LeaderIndex < LeaderTransforms.Num())
					{
						BoneTransforms[BoneIndex] = LeaderTransforms[LeaderIndex];
						bFoundLeader = true;
					}
				}

				if (!bFoundLeader && SkeletalMesh)
				{
					const int32 ParentIndex = SkeletalMesh->GetRefSkeleton().GetParentIndex(BoneIndex);

					BoneTransforms[BoneIndex] =
						BoneTransforms.IsValidIndex(ParentIndex) && ParentIndex < BoneIndex ?
						BoneTransforms[ParentIndex] * SkeletalMesh->GetRefSkeleton().GetRefBonePose()[BoneIndex] :
						SkeletalMesh->GetRefSkeleton().GetRefBonePose()[BoneIndex];
				}
			}
		}
	}
	else
	{
		BoneTransforms = InComponent->GetComponentSpaceTransforms();
	}
}

void FClothingSimulationContextCommon::FillRefToLocals(const USkeletalMeshComponent* InComponent, bool bIsInitialization)
{
	RefToLocals.Reset();

	// Constraints are initialized using bone distances upon initialization, so fill out reference pose
	if (bIsInitialization)
	{
		const USkeletalMesh* const SkeletalMesh = InComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh)
		{
			const int32 NumBones = SkeletalMesh->GetRefSkeleton().GetNum();
			RefToLocals.AddUninitialized(NumBones);
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				RefToLocals[BoneIndex] = FMatrix44f::Identity;
			}
		}
		return;
	}

	InComponent->GetCurrentRefToLocalMatrices(RefToLocals, InComponent->GetPredictedLODLevel());
}

void FClothingSimulationContextCommon::FillComponentToWorld(const USkeletalMeshComponent* InComponent)
{
	ComponentToWorld = InComponent->GetComponentTransform();
}

void FClothingSimulationContextCommon::FillWorldGravity(const USkeletalMeshComponent* InComponent)
{
	const UWorld* const ComponentWorld = InComponent->GetWorld();
	check(ComponentWorld);
	WorldGravity = FVector(0.f, 0.f, ComponentWorld->GetGravityZ());
}

void FClothingSimulationContextCommon::FillWindVelocity(const USkeletalMeshComponent* InComponent)
{
	InComponent->GetWindForCloth_GameThread(WindVelocity, WindAdaption);
}

void FClothingSimulationContextCommon::FillDeltaSeconds(float InDeltaSeconds, float InMaxPhysicsDelta)
{
	DeltaSeconds = FMath::Min(InDeltaSeconds, InMaxPhysicsDelta);
}

void FClothingSimulationContextCommon::FillTeleportMode(const USkeletalMeshComponent* InComponent, float InDeltaSeconds, float InMaxPhysicsDelta)
{
	TeleportMode = (InComponent->ClothTeleportMode < EClothingTeleportMode::Teleport && (InDeltaSeconds > InMaxPhysicsDelta * GClothMaxDeltaTimeTeleportMultiplier.GetValueOnGameThread())) ?
		EClothingTeleportMode::Teleport :
		InComponent->ClothTeleportMode;

	const float MaxVelocityScale = FMath::Clamp(GClothMaxVelocityScale.GetValueOnGameThread(), 0.f, 1.f);
	const float ComponentVelocityScale = InComponent->ClothVelocityScale;

	VelocityScale = (TeleportMode == EClothingTeleportMode::None && InDeltaSeconds > 0.f) ?
		FMath::Clamp(ComponentVelocityScale, 0.f, MaxVelocityScale) * FMath::Min(InDeltaSeconds, InMaxPhysicsDelta) / InDeltaSeconds : 0.f;
}

void FClothingSimulationContextCommon::FillMaxDistanceScale(const USkeletalMeshComponent* InComponent)
{
	MaxDistanceScale = InComponent->GetClothMaxDistanceScale();
}

void FClothingSimulationContextCommon::FillSolverGeometryScale(const USkeletalMeshComponent* InComponent)
{
	SolverGeometryScale = InComponent->ClothGeometryScale;
}

//==============================================================================
// FClothingSimulationCommon
//==============================================================================

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
FClothingSimulationCommon::FClothingSimulationCommon()
{
	MaxPhysicsDelta = UPhysicsSettings::Get()->MaxPhysicsDeltaTime;
}

FClothingSimulationCommon::~FClothingSimulationCommon()
{}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FClothingSimulationCommon::FillContextAndPrepareTick(const USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly)
{
	check(InOutContext);
	FClothingSimulationContextCommon* const Context = static_cast<FClothingSimulationContextCommon*>(InOutContext);

	Context->Fill(InComponent, InDeltaTime, MaxPhysicsDelta, bIsInitialization);

	// Checking the component here to track rare issue leading to invalid contexts
	if (!IsValid(InComponent))
	{
		const AActor* const CompOwner = InComponent->GetOwner();
		UE_LOGF(LogSkeletalMesh, Warning, 
			"Attempting to fill a clothing simulation context for a PendingKill skeletal mesh component (Comp: %ls, Actor: %ls). "
				"Pending kill skeletal mesh components should be unregistered before marked pending kill.", 
			*InComponent->GetName(), CompOwner ? *CompOwner->GetName() : TEXT("None"));

		// Make sure we clear this out to skip any attempted simulations
		Context->BoneTransforms.Reset();
	}

	if (Context->BoneTransforms.Num() == 0)
	{
		const AActor* const CompOwner = InComponent->GetOwner();
		const USkinnedMeshComponent* const Leader = InComponent->LeaderPoseComponent.Get();
		UE_LOGF(LogSkeletalMesh, Warning, "Attempting to fill a clothing simulation context for a skeletal mesh component that has zero bones (Comp: %ls, Leader: %ls, Actor: %ls).", *InComponent->GetName(), Leader ? *Leader->GetName() : TEXT("None"), CompOwner ? *CompOwner->GetName() : TEXT("None"));

		// Make sure we clear this out to skip any attempted simulations
		Context->BoneTransforms.Reset();
	}
}
