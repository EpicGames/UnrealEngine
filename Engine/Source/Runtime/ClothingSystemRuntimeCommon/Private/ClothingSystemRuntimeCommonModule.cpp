// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSystemRuntimeCommonModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FClothingSystemRuntimeCommonModule, ClothingSystemRuntimeCommon);

FClothingSystemRuntimeCommonModule::FClothingSystemRuntimeCommonModule()
{
}

void FClothingSystemRuntimeCommonModule::StartupModule()
{
}

void FClothingSystemRuntimeCommonModule::ShutdownModule()
{
}
