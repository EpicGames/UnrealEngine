// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/ClothingMeshUtils.h"

#include "ClothPhysicalMeshData.h"

#include "Math/UnrealMathUtility.h"
#include "Logging/LogMacros.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopedSlowTask.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

DEFINE_LOG_CATEGORY(LogClothingMeshUtils)

#define LOCTEXT_NAMESPACE "ClothingMeshUtils"

// This must match NUM_INFLUENCES_PER_VERTEX in GpuSkinCacheComputeShader.usf and GpuSkinVertexFactory.ush
// TODO: Make this easier to change in without messing things up
#define NUM_INFLUENCES_PER_VERTEX 5

namespace ClothingMeshUtils
{
	struct ClothMeshDesc::FClothBvEntry
	{
		const ClothMeshDesc* TmData;
		int32 Index;

		bool HasBoundingBox() const { return true; }

		Chaos::FAABB3 BoundingBox() const
		{
			int32 TriBaseIdx = Index * 3;

			const uint32 IA = TmData->Indices[TriBaseIdx + 0];
			const uint32 IB = TmData->Indices[TriBaseIdx + 1];
			const uint32 IC = TmData->Indices[TriBaseIdx + 2];

			const FVector3f& A = TmData->Positions[IA];
			const FVector3f& B = TmData->Positions[IB];
			const FVector3f& C = TmData->Positions[IC];

			Chaos::FAABB3 Bounds(A, A);

			Bounds.GrowToInclude(B);
			Bounds.GrowToInclude(C);

			return Bounds;
		}

		template<typename TPayloadType>
		int32 GetPayload(int32 Idx) const { return Idx; }
	};

	void ClothMeshDesc::GatherAllSourceTrianglesForTargetVertex(const TArray<FMeshToMeshFilterSet>& FilterSets, int32 TargetVertex) const
	{
		TSet<int32> FilteredTriangleSet;
		FilteredTriangleSet.Reserve(FilteredTriangles.Num());  // Reuse the previous number of filtered triangles as a guesstimate of the required filter size

		for (const FMeshToMeshFilterSet& FilterSet : FilterSets)
		{
			if (FilterSet.TargetVertices.Contains(TargetVertex))
			{
				FilteredTriangleSet.Append(FilterSet.SourceTriangles);
			}
		}

		if (FilteredTriangles.Num() != 0 || FilteredTriangleSet.Num() != 0)
		{
			bHasValidBVH = false;  // The set has likely changed, invalidate the BVH
		}

		FilteredTriangles = FilteredTriangleSet.Array();
	}

	float ClothMeshDesc::DistanceToTriangle(const FVector& Position, int32 TriangleBaseIndex) const
	{
		const uint32 IA = Indices[TriangleBaseIndex + 0];
		const uint32 IB = Indices[TriangleBaseIndex + 1];
		const uint32 IC = Indices[TriangleBaseIndex + 2];

		const FVector& A = (FVector)Positions[IA];
		const FVector& B = (FVector)Positions[IB];
		const FVector& C = (FVector)Positions[IC];

		const FVector PointOnTri = FMath::ClosestPointOnTriangleToPoint(Position, A, B, C);
		return (PointOnTri - Position).Size();
	}

	TArray<int32> ClothMeshDesc::FindCandidateTriangles(const FVector& InPoint, float InTolerance) const
	{
		ensure(HasValidMesh());
		static const int32 MinNumTrianglesForBVHCreation = 100;

		const int32 NumFilteredTriangles = FilteredTriangles.Num();

		const int32 NumTris = NumFilteredTriangles ? NumFilteredTriangles : Indices.Num() / 3;

		if (!NumFilteredTriangles && NumTris > MinNumTrianglesForBVHCreation)  // Don't use the BVH when using filtered triangle sets  TODO: Fix this
		{
			// This is not thread safe
			if (!bHasValidBVH)
			{
				TArray<FClothBvEntry> BVEntries;
				BVEntries.Reset(NumTris);

				if (NumFilteredTriangles)
				{
					for (int32 Tri : FilteredTriangles)
					{
						BVEntries.Add({ this, Tri });
					}
				}
				else
				{
					for (int32 Tri = 0; Tri < NumTris; ++Tri)
					{
						BVEntries.Add({ this, Tri });
					}
				}
				BVH.Reinitialize(BVEntries);
				bHasValidBVH = true;
			}
			Chaos::FAABB3 TmpAABB(InPoint, InPoint);
			TmpAABB.Thicken(InTolerance);  // Most points might be very close to the triangle, but not directly on it
			TArray<int32> Triangles = BVH.FindAllIntersections(TmpAABB);

			// Refine the search to include all nearby bounded volumes (the point could well be outside the closest triangle's bounded volume)
			if (Triangles.Num())
			{
				float ClosestDistance = TNumericLimits<float>::Max();
				for (const int32 Triangle : Triangles)
				{
					ClosestDistance = FMath::Min(ClosestDistance, DistanceToTriangle(InPoint, Triangle * 3));
				}

				TmpAABB.Thicken(ClosestDistance);
				return BVH.FindAllIntersections(TmpAABB);
			}
		}
		return NumFilteredTriangles ? FilteredTriangles : TArray<int32>();
	}

	TConstArrayView<float> ClothMeshDesc::GetMaxEdgeLengths() const
	{
		if (!MaxEdgeLengths.Num())
		{
			ensure(Indices.Num() % 3 == 0);  // Check we have properly formed triangles

			const int32 NumMesh0Verts = Positions.Num();
			const int32 NumMesh0Tris = Indices.Num() / 3;

			MaxEdgeLengths.Init(0.f, NumMesh0Verts);

			for (int32 TriangleIdx = 0; TriangleIdx < NumMesh0Tris; ++TriangleIdx)
			{
				const uint32* const Triangle = &Indices[TriangleIdx * 3];

				for (int32 Vertex0Idx = 0; Vertex0Idx < 3; ++Vertex0Idx)
				{
					const int32 Vertex1Idx = (Vertex0Idx + 1) % 3;

					const FVector& P0 = (FVector)Positions[Triangle[Vertex0Idx]];
					const FVector& P1 = (FVector)Positions[Triangle[Vertex1Idx]];

					const float EdgeLength = FVector::Distance(P0, P1);
					MaxEdgeLengths[Triangle[Vertex0Idx]] = FMath::Max(MaxEdgeLengths[Triangle[Vertex0Idx]], EdgeLength);
					MaxEdgeLengths[Triangle[Vertex1Idx]] = FMath::Max(MaxEdgeLengths[Triangle[Vertex1Idx]], EdgeLength);
				}
			}
		}
		return TConstArrayView<float>(MaxEdgeLengths);
	}

	void ClothMeshDesc::ComputeAveragedNormals()
	{
		AveragedNormals.Init(FVector3f::ZeroVector, Positions.Num());

		const int32 NumTriangles = Indices.Num() / 3;
		for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
		{
			const uint32 Index0 = Indices[3 * TriangleIndex + 0];
			const uint32 Index1 = Indices[3 * TriangleIndex + 1];
			const uint32 Index2 = Indices[3 * TriangleIndex + 2];
			const FVector3f& Position0 = Positions[Index0];
			const FVector3f& Position1 = Positions[Index1];
			const FVector3f& Position2 = Positions[Index2];

			FVector3f Normal = FVector3f::CrossProduct(Position2 - Position0, Position1 - Position0);
			if (Normal.Normalize())  // Skip contributions from degenerate triangles
			{
				AveragedNormals[Index0] += Normal;
				AveragedNormals[Index1] += Normal;
				AveragedNormals[Index2] += Normal;
			}
		}

		for (int32 Index = 0; Index < AveragedNormals.Num(); ++Index)
		{
			FVector3f& Normal = AveragedNormals[Index];
			if (!Normal.Normalize())
			{
				Normal = FVector3f::XAxisVector;
			}
		}
	}

	// inline function used to force the unrolling of the skinning loop
	FORCEINLINE static void AddInfluence(FVector3f& OutPosition, FVector3f& OutNormal, const FVector3f& RefParticle, const FVector3f& RefNormal, const FMatrix44f& BoneMatrix, const float Weight)
	{
		OutPosition += BoneMatrix.TransformPosition(RefParticle) * Weight;
		OutNormal += BoneMatrix.TransformVector(RefNormal) * Weight;
	}

	void SkinPhysicsMesh(
		const TArray<int32>& InBoneMap,
		const FClothPhysicalMeshData& InMesh,
		const FTransform& PostTransform,
		const FMatrix44f* InBoneMatrices,
		const int32 InNumBoneMatrices,
		TArray<FVector3f>& OutPositions,
		TArray<FVector3f>& OutNormals)
	{
		const uint32 NumVerts = InMesh.Vertices.Num();

		OutPositions.Reset(NumVerts);
		OutNormals.Reset(NumVerts);
		OutPositions.AddZeroed(NumVerts);
		OutNormals.AddZeroed(NumVerts);

		const int32 MaxInfluences = InMesh.MaxBoneWeights;
		UE_CLOGF(MaxInfluences > 12, LogClothingMeshUtils, Warning, "The cloth physics mesh skinning code can't cope with more than 12 bone influences.");

		const int32* const RESTRICT BoneMap = InBoneMap.GetData();  // Remove RangeCheck for faster skinning in development builds
		const FMatrix44f* const RESTRICT BoneMatrices = InBoneMatrices;
		
		static const uint32 MinParallelVertices = 500;  // 500 seems to be the lowest threshold still giving gains even on profiled assets that are only using a small number of influences

		ParallelFor(NumVerts, [&InMesh, &PostTransform, BoneMap, BoneMatrices, &OutPositions, &OutNormals](uint32 VertIndex)
		{
			// Fixed particle, needs to be skinned
			const uint16* const RESTRICT BoneIndices = InMesh.BoneData[VertIndex].BoneIndices;
			const float* const RESTRICT BoneWeights = InMesh.BoneData[VertIndex].BoneWeights;

			// WARNING - HORRIBLE UNROLLED LOOP + JUMP TABLE BELOW
			// done this way because this is a pretty tight and perf critical loop. essentially
			// rather than checking each influence we can just jump into this switch and fall through
			// everything to compose the final skinned data
			const FVector3f& RefParticle = InMesh.Vertices[VertIndex];
			const FVector3f& RefNormal = InMesh.Normals[VertIndex];
			FVector3f& OutPosition = OutPositions[VertIndex];
			FVector3f& OutNormal = OutNormals[VertIndex];
			switch (InMesh.BoneData[VertIndex].NumInfluences)
			{
			case 12: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[11]]], BoneWeights[11]);  // Intentional fall through
			case 11: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[10]]], BoneWeights[10]);  // Intentional fall through
			case 10: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 9]]], BoneWeights[ 9]);  // Intentional fall through
			case  9: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 8]]], BoneWeights[ 8]);  // Intentional fall through
			case  8: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 7]]], BoneWeights[ 7]);  // Intentional fall through
			case  7: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 6]]], BoneWeights[ 6]);  // Intentional fall through
			case  6: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 5]]], BoneWeights[ 5]);  // Intentional fall through
			case  5: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 4]]], BoneWeights[ 4]);  // Intentional fall through
			case  4: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 3]]], BoneWeights[ 3]);  // Intentional fall through
			case  3: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 2]]], BoneWeights[ 2]);  // Intentional fall through
			case  2: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 1]]], BoneWeights[ 1]);  // Intentional fall through
			case  1: AddInfluence(OutPosition, OutNormal, RefParticle, RefNormal, BoneMatrices[BoneMap[BoneIndices[ 0]]], BoneWeights[ 0]);  // Intentional fall through
			default: break;
			}

			// Ignore any user scale. It's already accounted for in our skinning matrices
			// This is the use case for NVcloth
			FTransform PostTransformInternal = PostTransform;
			PostTransformInternal.SetScale3D(FVector(1.0f));

			OutPosition = (FVector3f)PostTransformInternal.InverseTransformPosition((FVector)OutPosition);
			OutNormal = (FVector3f)PostTransformInternal.InverseTransformVector((FVector)OutNormal);

			if (OutNormal.SizeSquared() > SMALL_NUMBER)
			{
				OutNormal = OutNormal.GetUnsafeNormal();
			}
		}, NumVerts > MinParallelVertices ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
	}

	/** 
	 * Gets the best match triangle for a specified position from the triangles in Mesh.
	 * Performs no validation on the incoming mesh data, the mesh data should be verified
	 * to be valid before using this function
	 */
	static int32 GetBestTriangleBaseIndex(const ClothMeshDesc& Mesh, const FVector& Position, float InTolerance = KINDA_SMALL_NUMBER)
	{
		float MinimumDistanceSq = MAX_flt;
		int32 ClosestBaseIndex = INDEX_NONE;

		const TArray<int32> Tris = Mesh.FindCandidateTriangles(Position, InTolerance);
		int32 NumTriangles = Tris.Num();
		if (!NumTriangles)
		{
			NumTriangles = Mesh.GetIndices().Num() / 3;
		}
		for (int32 TriIdx = 0; TriIdx < NumTriangles; ++TriIdx)
		{
			int32 TriBaseIdx = (Tris.Num() ? Tris[TriIdx] : TriIdx) * 3;

			const uint32 IA = Mesh.GetIndices()[TriBaseIdx + 0];
			const uint32 IB = Mesh.GetIndices()[TriBaseIdx + 1];
			const uint32 IC = Mesh.GetIndices()[TriBaseIdx + 2];

			const FVector& A = (FVector)Mesh.GetPositions()[IA];
			const FVector& B = (FVector)Mesh.GetPositions()[IB];
			const FVector& C = (FVector)Mesh.GetPositions()[IC];

			FVector PointOnTri = FMath::ClosestPointOnTriangleToPoint(Position, A, B, C);
			float DistSq = (PointOnTri - Position).SizeSquared();

			if (DistSq < MinimumDistanceSq)
			{
				MinimumDistanceSq = DistSq;
				ClosestBaseIndex = TriBaseIdx;
			}
		}

		return ClosestBaseIndex;
	}

	namespace  // Helpers
	{
		using TriangleDistance = TPair<int32, float>;

		/** Similar to GetBestTriangleBaseIndex but returns the N closest triangles. */
		template<uint32 N>
		TStaticArray<TriangleDistance, N> GetNBestTrianglesBaseIndices(const ClothMeshDesc& Mesh, const FVector& Position)
		{
			const TArray<int32> Tris = Mesh.FindCandidateTriangles(Position);
			int32 NumTriangles = Tris.Num();

			const TArray<int32>& FilteredTriangles = Mesh.GetFilteredTriangles();
			const int32 NumFilteredTriangles = FilteredTriangles.Num();
			const bool bHasFilteredTriangles = (NumFilteredTriangles > 0);

			auto GetTriangleIndex = [bHasFilteredTriangles, &FilteredTriangles](int32 Index) -> int32
				{
					return bHasFilteredTriangles ? FilteredTriangles[Index] : Index;
				};

			if (NumTriangles < N)
			{
				// Couldn't find N candidates using FindCandidateTriangles. Grab all triangles in the mesh and see if
				// we have enough

				NumTriangles = bHasFilteredTriangles ? NumFilteredTriangles : Mesh.GetIndices().Num() / 3;

				if (NumTriangles < N)
				{
					// The mesh doesn't have N triangles. Get as many as we can.
					TStaticArray<TriangleDistance, N> ClosestTriangles;

					int32 i = 0;
					for ( ; i < NumTriangles; ++i)
					{
						const int32 TriBaseIdx = 3 * GetTriangleIndex(i);
						const float CurrentDistance = Mesh.DistanceToTriangle(Position, TriBaseIdx);
						ClosestTriangles[i] = TPair<int32, float>{ TriBaseIdx, CurrentDistance };
					}

					// fill the rest with INDEX_NONE 
					for (; i < N; ++i)
					{
						ClosestTriangles[i] = TPair<int32, float>{ INDEX_NONE, 0.0f };
					}

					return ClosestTriangles;
				}
			}

			check(NumTriangles >= N);

			// Closest N triangle, heapified so that the first triangle is the furthest of the N closest triangles
			TStaticArray<TriangleDistance, N> ClosestTriangles;

			// Get the distances to the first N triangles (unsorted)
			int32 i = 0;
			for (; i < N; ++i)
			{
				int32 TriBaseIdx = (Tris.Num() >= N ? Tris[i] : GetTriangleIndex(i)) * 3;
				float CurrentDistance = Mesh.DistanceToTriangle(Position, TriBaseIdx);
				ClosestTriangles[i] = TPair<int32, float>{ TriBaseIdx, CurrentDistance };
			}

			// Max-heapify the N first triangle distances
			Algo::Heapify(ClosestTriangles, [](const TriangleDistance& A, const TriangleDistance& B)
			{
				return A.Value > B.Value;
			});

			// Now keep going
			for (; i < NumTriangles; ++i)
			{
				int32 TriBaseIdx = (Tris.Num() >= N ? Tris[i] : GetTriangleIndex(i)) * 3;
				float CurrentDistance = Mesh.DistanceToTriangle(Position, TriBaseIdx);

				if (CurrentDistance < ClosestTriangles[0].Value)
				{
					// Triangle is closer than the "furthest" ClosestTriangles

					// Replace the furthest ClosestTriangles with this triangle...
					ClosestTriangles[0] = TPair<int32, float>{ TriBaseIdx, CurrentDistance };

					// ...and re-heapify the closest triangles
					Algo::Heapify(ClosestTriangles, [](const TriangleDistance& A, const TriangleDistance& B)
					{
						return A.Value > B.Value;
					});
				}
			}

			return ClosestTriangles;
		}

		// Using this formula, for R = Distance / MaxDistance:
		//		Weight = 1 - 3 * R ^ 2 + 3 * R ^ 4 - R ^ 6 = (1-R^2)^3
		// From the Houdini metaballs docs: https://www.sidefx.com/docs/houdini/nodes/sop/metaball.html#kernels
		// Which was linked to from the cloth capture doc: https://www.sidefx.com/docs/houdini/nodes/sop/clothcapture.html
		float Kernel(float Distance, float MaxDistance)
		{
			if (MaxDistance == 0.0f)
			{
				return 0.0f;
			}

			float R = FMath::Max(0.0f, FMath::Min(1.0f, Distance / MaxDistance));
			float OneMinusR2 = 1.0f - R * R;
			return OneMinusR2 * OneMinusR2 * OneMinusR2;
		}

		// Return false if triangle specified by ClosestTriangleBaseIdx is degenerate, true otherwise
		bool SingleSkinningDataForVertex(const FVector3f& VertPosition,
										 const FVector3f& VertNormal,
										 const FVector3f& VertTangent,
										 const ClothMeshDesc& SourceMesh,
										 int32 ClosestTriangleBaseIdx,
										 FMeshToMeshVertData& OutSkinningData)
		{
			check(SourceMesh.HasAveragedNormals());

			const FVector3f& A = SourceMesh.GetPositions()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx]];
			const FVector3f& B = SourceMesh.GetPositions()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1]];
			const FVector3f& C = SourceMesh.GetPositions()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2]];

			const FVector3f& NA = SourceMesh.GetNormals()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx]];
			const FVector3f& NB = SourceMesh.GetNormals()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1]];
			const FVector3f& NC = SourceMesh.GetNormals()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2]];

			// Before generating the skinning data we need to check for a degenerate triangle.
			// If we find _any_ degenerate triangles we will notify and fail to generate the skinning data
			const FVector3f TriNormal = FVector3f::CrossProduct(B - A, C - A);
			if (TriNormal.SizeSquared() < SMALL_NUMBER)
			{
				// Failed, we have 2 identical vertices

				// Log
				const uint32 IndexA = SourceMesh.GetIndices()[ClosestTriangleBaseIdx];
				const uint32 IndexB = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1];
				const uint32 IndexC = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2];
				FText Error = FText::Format(LOCTEXT("DegenerateTriangleError", "Failed to generate skinning data, found coincident vertices in triangle ({0}, {1}, {2}), points A={3} B={4} C={5}"),
					IndexA, IndexB, IndexC,
					FText::FromString(A.ToString()), FText::FromString(B.ToString()), FText::FromString(C.ToString()));
				UE_LOGF(LogClothingMeshUtils, Warning, "%ls", *Error.ToString());

				return false;
			}

			OutSkinningData.PositionBaryCoordsAndDist = GetPointBaryAndDistWithNormals(A, B, C, NA, NB, NC, VertPosition);
			OutSkinningData.NormalBaryCoordsAndDist = GetPointBaryAndDistWithNormals(A, B, C, NA, NB, NC, VertPosition + VertNormal);
			OutSkinningData.TangentBaryCoordsAndDist = GetPointBaryAndDistWithNormals(A, B, C, NA, NB, NC, VertPosition + VertTangent);
			OutSkinningData.SourceMeshVertIndices[0] = SourceMesh.GetIndices()[ClosestTriangleBaseIdx];
			OutSkinningData.SourceMeshVertIndices[1] = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1];
			OutSkinningData.SourceMeshVertIndices[2] = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2];
			OutSkinningData.SourceMeshVertIndices[3] = 0;
			OutSkinningData.Weight = 1.0f;

			return true;
		}


		template<unsigned int NUM_INFLUENCES>
		bool MultipleSkinningDataForVertex(TStaticArray<FMeshToMeshVertData, NUM_INFLUENCES>& SkinningData,
										   const ClothMeshDesc& TargetMesh,
										   const ClothMeshDesc& SourceMesh,
										   int32 VertIdx0,
										   float KernelMaxDistance,
										   float MaxIncidentEdgeLength)
		{
			check(SourceMesh.HasAveragedNormals());

			const FVector3f& VertPosition = TargetMesh.GetPositions()[VertIdx0];
			const FVector3f& VertNormal = TargetMesh.GetNormals()[VertIdx0];

			FVector VertTangent;
			if (TargetMesh.HasTangents())
			{
				VertTangent = (FVector)TargetMesh.GetTangents()[VertIdx0];
			}
			else
			{
				FVector3f Tan0, Tan1;
				VertNormal.FindBestAxisVectors(Tan0, Tan1);
				VertTangent = (FVector)Tan0;
			}

			TStaticArray<TriangleDistance, NUM_INFLUENCES> NearestTriangles =
				GetNBestTrianglesBaseIndices<NUM_INFLUENCES>(SourceMesh, (FVector)VertPosition);

			float SumWeight = 0.0f;

			for (int j = 0; j < NUM_INFLUENCES; ++j)
			{
				FMeshToMeshVertData& CurrentData = SkinningData[j];

				int ClosestTriangleBaseIdx = NearestTriangles[j].Key;
				if (ClosestTriangleBaseIdx == INDEX_NONE)
				{
					CurrentData.Weight = 0.0f;
					CurrentData.SourceMeshVertIndices[3] = 0xFFFF;
					continue;
				}

				const FVector3f& A = SourceMesh.GetPositions()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx]];
				const FVector3f& B = SourceMesh.GetPositions()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1]];
				const FVector3f& C = SourceMesh.GetPositions()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2]];

				const FVector3f& NA = SourceMesh.GetNormals()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx]];
				const FVector3f& NB = SourceMesh.GetNormals()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1]];
				const FVector3f& NC = SourceMesh.GetNormals()[SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2]];

				// Before generating the skinning data we need to check for a degenerate triangle.
				// If we find _any_ degenerate triangles we will notify and fail to generate the skinning data
				const FVector3f TriNormal = FVector3f::CrossProduct(B - A, C - A);
				if (TriNormal.SizeSquared() < SMALL_NUMBER)
				{
					// Failed, we have 2 identical vertices

					const uint32 IndexA = SourceMesh.GetIndices()[ClosestTriangleBaseIdx];
					const uint32 IndexB = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1];
					const uint32 IndexC = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2];
					FText Error = FText::Format(LOCTEXT("DegenerateTriangleErrorMultipleInfluences", "Failed to generate skinning data, found coincident vertices in triangle ({0}, {1}, {2}), points A={3} B={4} C={5}"),
						IndexA, IndexB, IndexC,
						FText::FromString(A.ToString()), FText::FromString(B.ToString()), FText::FromString(C.ToString()));
					UE_LOGF(LogClothingMeshUtils, Warning, "%ls", *Error.ToString());

					return false;
				}

				CurrentData.PositionBaryCoordsAndDist = GetPointBaryAndDistWithNormals(A, B, C, NA, NB, NC, VertPosition);
				CurrentData.NormalBaryCoordsAndDist = GetPointBaryAndDistWithNormals(A, B, C, NA, NB, NC, VertPosition + VertNormal);
				CurrentData.TangentBaryCoordsAndDist = GetPointBaryAndDistWithNormals(A, B, C, NA, NB, NC, VertPosition + (FVector3f)VertTangent);
				CurrentData.SourceMeshVertIndices[0] = SourceMesh.GetIndices()[ClosestTriangleBaseIdx];
				CurrentData.SourceMeshVertIndices[1] = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 1];
				CurrentData.SourceMeshVertIndices[2] = SourceMesh.GetIndices()[ClosestTriangleBaseIdx + 2];
				CurrentData.SourceMeshVertIndices[3] = 0;

				CurrentData.Weight = Kernel(NearestTriangles[j].Value, KernelMaxDistance);
				SumWeight += CurrentData.Weight;
			}

			// Normalize weights
			if (SumWeight > 0.0f)
			{
				for (FMeshToMeshVertData& CurrentData : SkinningData)
				{
					CurrentData.Weight /= SumWeight;
				}
			}

			return true;
		}

		void FixZeroWeightVertices(
			TArray<FMeshToMeshVertData>& InOutSkinningData,
			const ClothMeshDesc& TargetMesh,
			const ClothMeshDesc& SourceMesh,
			const TArray<FMeshToMeshFilterSet>& FilterSets)
		{
			if (!ensure(InOutSkinningData.Num() > 0))
			{
				return;
			}

			const int32 NumTargetMeshVerts = TargetMesh.GetPositions().Num();

			if (!ensure(NumTargetMeshVerts * NUM_INFLUENCES_PER_VERTEX == InOutSkinningData.Num()))
			{
				return;
			}

			const TConstArrayView<float> MaxEdgeLengths = TargetMesh.GetMaxEdgeLengths();
			check(NumTargetMeshVerts == MaxEdgeLengths.Num());

			bool bAnySmallTriangleEncountered = false;

			for (int32 VID = 0; VID < NumTargetMeshVerts; ++VID)
			{
				float SumWeight = 0.0f;
				for (int Inf = 0; Inf < NUM_INFLUENCES_PER_VERTEX; ++Inf)
				{
					const FMeshToMeshVertData& Data = InOutSkinningData[NUM_INFLUENCES_PER_VERTEX * VID + Inf];
					if (Data.SourceMeshVertIndices[3] < 0xFFFF)
					{
						SumWeight += Data.Weight;
					}
				}

				if (SumWeight < KINDA_SMALL_NUMBER)
				{
					// Fall back to single influence
					const FVector3f& VertPosition = TargetMesh.GetPositions()[VID];
					const FVector3f& VertNormal = TargetMesh.GetNormals()[VID];

					FVector3f VertTangent;
					if (TargetMesh.HasTangents())
					{
						VertTangent = TargetMesh.GetTangents()[VID];
					}
					else
					{
						FVector3f Tan0, Tan1;
						VertNormal.FindBestAxisVectors(Tan0, Tan1);
						VertTangent = Tan0;
					}

					// Compute single-influence attachment for the first skinning data
					FMeshToMeshVertData& FirstData = InOutSkinningData[NUM_INFLUENCES_PER_VERTEX * VID];

					const int32 ClosestTriangleBaseIdx = ClothingMeshUtils::GetBestTriangleBaseIndex(SourceMesh, (FVector)VertPosition, MaxEdgeLengths[VID]);
					if (!ensure(ClosestTriangleBaseIdx != INDEX_NONE))
					{
						FirstData.Weight = 0.0f;
					}
					else
					{
						const uint16 PreviousFlag = FirstData.SourceMeshVertIndices[3];
						const bool bSkinningSuccess = SingleSkinningDataForVertex(VertPosition, VertNormal, VertTangent, SourceMesh, ClosestTriangleBaseIdx, FirstData);
						FirstData.SourceMeshVertIndices[3] = PreviousFlag;

						if (!bSkinningSuccess)
						{
							bAnySmallTriangleEncountered = true;
						}
					}

					// Set all other skinning data to have zero weight
					for (int Inf = 1; Inf < NUM_INFLUENCES_PER_VERTEX; ++Inf)
					{
						FMeshToMeshVertData& CurrentData = InOutSkinningData[NUM_INFLUENCES_PER_VERTEX * VID + Inf];
						CurrentData.SourceMeshVertIndices[3] = 0xFFFF;
						CurrentData.Weight = 0.0f;
					}
				}
			}

#if WITH_EDITOR
			if (bAnySmallTriangleEncountered)
			{
				const FText ErrorMsg = LOCTEXT("DegenerateTriangleErrorToast", "Failed to generate skinning data, found conincident vertices in at least one triangle. See Log for details");
				FNotificationInfo Info(ErrorMsg);
				Info.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
#endif

		}

	}  // Anonymous namespace

	void GenerateMeshToMeshVertData(
		TArray<FMeshToMeshVertData>& OutMeshToMeshVertData,
		const ClothMeshDesc& TargetMesh,
		const ClothMeshDesc& SourceMesh,
		const FPointWeightMap* MaxDistances,
		bool bUseSmoothTransitions,
		bool bUseMultipleInfluences,
		float KernelMaxDistance,
		const TArray<FMeshToMeshFilterSet>& FilterSets)
	{
		check(SourceMesh.HasAveragedNormals());

		OutMeshToMeshVertData.Reset();

		if (!SourceMesh.GetIndices().Num() || !TargetMesh.GetPositions().Num())
		{
			return; // Empty meshes must return an empty mapping and no warning
		}

		if (!TargetMesh.HasValidMesh())  // Check that the number of positions is equal to the number of normals and that the number of indices is divisible by 3
		{
			UE_LOGF(LogClothingMeshUtils, Warning, "Failed to generate mesh to mesh skinning data. Invalid Target Mesh.");
			return;
		}

		if (!SourceMesh.HasValidMesh())  // Check that the number of positions is equal to the number of normals and that the number of indices is divisible by 3
		{
			UE_LOGF(LogClothingMeshUtils, Warning, "Failed to generate mesh to mesh skinning data. Invalid Source Mesh.");
			return;
		}

		const int32 NumMesh0Verts = TargetMesh.GetPositions().Num();

		const TConstArrayView<float> MaxEdgeLengths = TargetMesh.GetMaxEdgeLengths();
		check(NumMesh0Verts == MaxEdgeLengths.Num());

		constexpr int32 SlowTaskDivider = 100;
		const float NumSteps = (float)(NumMesh0Verts / SlowTaskDivider);
		FScopedSlowTask SlowTask(NumSteps, LOCTEXT("GenerateMeshToMeshVertData", "Generating cloth deformer data..."));
		SlowTask.MakeDialogDelayed(1.f);

		if (bUseMultipleInfluences)
		{
			OutMeshToMeshVertData.Reserve(NumMesh0Verts * NUM_INFLUENCES_PER_VERTEX);

			// For all mesh0 verts
			for (int32 VertIdx0 = 0; VertIdx0 < NumMesh0Verts; ++VertIdx0)
			{
				SourceMesh.GatherAllSourceTrianglesForTargetVertex(FilterSets, VertIdx0);

				TStaticArray<FMeshToMeshVertData, NUM_INFLUENCES_PER_VERTEX> SkinningData;
				const bool bOK = MultipleSkinningDataForVertex(
					SkinningData,
					TargetMesh,
					SourceMesh,
					VertIdx0,
					KernelMaxDistance,
					MaxEdgeLengths[VertIdx0]);

				// If we find _any_ degenerate triangles we will notify and fail to generate the skinning data
				if (!bOK)
				{
					UE_LOGF(LogClothingMeshUtils, Warning, "Error generating mesh-to-mesh skinning data");
					OutMeshToMeshVertData.Reset();
					return;
				}

				OutMeshToMeshVertData.Append(SkinningData.GetData(), NUM_INFLUENCES_PER_VERTEX);

				if ((VertIdx0 + 1) % SlowTaskDivider == 0)
				{
					SlowTask.EnterProgressFrame();
				}
			}

			check(OutMeshToMeshVertData.Num() == NumMesh0Verts * NUM_INFLUENCES_PER_VERTEX);
		}
		else
		{
			OutMeshToMeshVertData.Reserve(NumMesh0Verts);

			// For all mesh0 verts
			bool bAnySmallTriangleEncountered = false;
			for (int32 VertIdx0 = 0; VertIdx0 < NumMesh0Verts; ++VertIdx0)
			{
				SourceMesh.GatherAllSourceTrianglesForTargetVertex(FilterSets, VertIdx0);

				OutMeshToMeshVertData.AddZeroed();
				FMeshToMeshVertData& SkinningData = OutMeshToMeshVertData.Last();

				const FVector3f& VertPosition = TargetMesh.GetPositions()[VertIdx0];
				const FVector3f& VertNormal = TargetMesh.GetNormals()[VertIdx0];

				FVector VertTangent;
				if (TargetMesh.HasTangents())
				{
					VertTangent = (FVector)TargetMesh.GetTangents()[VertIdx0];
				}
				else
				{
					FVector3f Tan0, Tan1;
					VertNormal.FindBestAxisVectors(Tan0, Tan1);
					VertTangent = (FVector)Tan0;
				}

				const int32 ClosestTriangleBaseIdx = GetBestTriangleBaseIndex(SourceMesh, (FVector)VertPosition, MaxEdgeLengths[VertIdx0]);
				check(ClosestTriangleBaseIdx != INDEX_NONE);

				const bool bSkinningSuccess = SingleSkinningDataForVertex(VertPosition, VertNormal, (FVector3f)VertTangent, SourceMesh, ClosestTriangleBaseIdx, SkinningData);
				
				if (!bSkinningSuccess)
				{
					bAnySmallTriangleEncountered = true;
				}

				if ((VertIdx0 + 1) % SlowTaskDivider == 0)
				{
					SlowTask.EnterProgressFrame();
				}
			}

			check(OutMeshToMeshVertData.Num() == NumMesh0Verts);

#if WITH_EDITOR
			if (bAnySmallTriangleEncountered)
			{
				const FText ErrorMsg = LOCTEXT("DegenerateTriangleErrorToast", "Failed to generate skinning data, found conincident vertices in at least one triangle. See Log for details");
				FNotificationInfo *Info = new FNotificationInfo(ErrorMsg);
				Info->ExpireDuration = 5.0f;
				// Queue notification because we may not be on the main game thread
				FSlateNotificationManager::Get().QueueNotification(Info);
			}
#endif

		}

		if (OutMeshToMeshVertData.Num())
		{
			if (bUseMultipleInfluences)
			{
				FixZeroWeightVertices(OutMeshToMeshVertData, TargetMesh, SourceMesh, FilterSets);
			}

			if (MaxDistances)
			{
				ComputeVertexContributions(OutMeshToMeshVertData, MaxDistances, bUseSmoothTransitions, bUseMultipleInfluences);
			}
		}
	}

	// TODO: Vertex normals are not used at present, a future improved algorithm might however
	FVector4f GetPointBaryAndDist(const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& Point)
	{
		FPlane4f TrianglePlane(A, B, C);
		const FVector3f PointOnTriPlane = FVector3f::PointPlaneProject(Point, TrianglePlane);
		const FVector3f BaryCoords = (FVector3f)FMath::ComputeBaryCentric2D((FVector)PointOnTriPlane, (FVector)A, (FVector)B, (FVector)C); // LWC_TODO: ComputeBaryCentric2D only supports FVector
		return FVector4f(BaryCoords, TrianglePlane.PlaneDot(Point)); // Note: The normal of the plane points away from the Clockwise face (instead of the counter clockwise face) in Left Handed Coordinates (This is why we need to invert the normals later on when before sending it to the shader)
	}


	
	double TripleProduct(const FVector3f& A, const FVector3f& B, const FVector3f& C)
	{
		return FVector3f::DotProduct(A, FVector3f::CrossProduct(B,C));
	}

	// Solve the equation x^2 + Ax + B = 0 for real roots. 
	// Requires an array of size 2 for the results. The return value is the number of results,
	// either 0 or 2.
	int32 QuadraticRoots(double A, double B, double Result[])
	{
		double D = 0.25 * A * A - B;
		if (D >= 0.0)
		{
			D = FMath::Sqrt(D);
			Result[0] = -0.5 * A - D;
			Result[1] = -0.5 * A + D; 
			return 2; 
		}
		return 0;
	}
	
	// Solve the equation x^3 + Ax^2 + Bx + C = 0 for real roots. Requires an array of size 3 
	// for the results. The return value is the number of results, ranging from 1 to 3.
	// Using Viete's trig formula. See: https://en.wikipedia.org/wiki/Cubic_equation
	int32 CubicRoots(double A, double B, double C, double Result[])
	{
		double A2 = A * A;
		double P = (A2 - 3.0 * B) / 9.0;
		double Q = (A * (2.0 * A2 - 9.0 * B) + 27.0 * C) / 54.0;
		double P3 = P * P * P;
		double Q2 = Q * Q;
		if (Q2 <= (P3 + SMALL_NUMBER))
		{
			// We have three real roots.
			double T = Q / FMath::Sqrt(P3);
			// FMath::Acos automatically clamps T to [-1,1]
			T = FMath::Acos(T);
			A /= 3.0;
			P = -2.0 * FMath::Sqrt(P);
			Result[0] = P * FMath::Cos(T / 3.0) - A;
			Result[1] = P * FMath::Cos((T + 2.0 * PI) / 3.0) - A;
			Result[2] = P * FMath::Cos((T - 2.0 * PI) / 3.0) - A;
			return 3;
		}
		else
		{
			// One or two real roots.
			double R_1 = FMath::Pow(FMath::Abs(Q) + FMath::Sqrt(Q2 - P3), 1.0 / 3.0);
			if (Q > 0.0)
			{
				R_1 = -R_1;
			}
			double R_2 = 0.0;
			if (!FMath::IsNearlyZero(A))
			{
				R_2 = P / R_1;
			}
			A /= 3.0;
			Result[0] = (R_1 + R_2) - A;
			
			if (!FMath::IsNearlyZero(UE_HALF_SQRT_3 * (R_1 - R_2)))
			{
				return 1;
			}
			
			// Yoda: No. There is another...
			Result[1] = -0.5 * (R_1 + R_2) - A;
			return 2;
		}
	}

	// Solve Ax^2 + Bx + C = 0 for real roots. Handle cases where coefficients are zero.
	static int32 QuadraticSolve(const double Coeffs[3], double Out[])
	{
		const double A = Coeffs[0], B = Coeffs[1], C = Coeffs[2];
		if (FMath::IsNearlyZero(A))
		{
			// First coefficient is zero, so this is at most linear

			if (FMath::IsNearlyZero(B))
			{
				// Second coefficient is also zero, uh-oh
				return 0;
			}
			else
			{
				// Linear Bx + C = 0 and B != 0.
				//     =>  x = -C/B
				Out[0] = -C / B;
				return 1;
			}
		}
		else
		{
			// Quadratic Ax^2 + Bx + C = 0 and A != 0
			//        =>  x^2 + (B/A)x + C/A = 0

			return QuadraticRoots(B / A, C / A, Out);
		}
	}

	// Solve Ax^3 + Bx^2 + Cx + D = 0 for real roots. Handle cases where coefficients are zero.
	static int32 CubicSolve(const double Coeffs[4], double Out[])
	{
		const double A = Coeffs[0], B = Coeffs[1], C = Coeffs[2], D = Coeffs[3];

		if (FMath::IsNearlyZero(A))
		{
			// Leading coefficient is zero, so this is at most quadratic
			double QCoeffs[3] = { B, C, D };
			return QuadraticSolve(QCoeffs, Out);
		}
		else
		{
			// The cubic solver operates on cubics of the form: x^3 + Rx^2 + Sx + T = 0
			return CubicRoots(B/A, C/A, D/A, Out);
		}
	}

	// Find mins and maxes of cubic function Ax^3 + Bx^2 + Cx + D
	// i.e. solve       3Ax^2 + 2Bx + C = 0
	static int32 CubicExtrema(double A, double B, double C, double Out[])
	{
		double Coeffs[3] = { 3.0 * A, 2.0 * B, C };
		return QuadraticSolve(Coeffs, Out);
	}

	constexpr int MaxNumRootsAndExtrema = 5;

	static int32 CoplanarityParam(
		const FVector3f& A, const FVector3f& B, const FVector3f& C,
		const FVector3f& OffsetA, const FVector3f& OffsetB, const FVector3f& OffsetC,
		const FVector3f& Point, double Out[MaxNumRootsAndExtrema])
	{
		FVector3f PA = A - Point;
		FVector3f PB = B - Point;
		FVector3f PC = C - Point;

		double Coeffs[4] = {
			TripleProduct(OffsetA, OffsetB, OffsetC),
			TripleProduct(PA, OffsetB, OffsetC) + TripleProduct(OffsetA, PB, OffsetC) + TripleProduct(OffsetA, OffsetB, PC),
			TripleProduct(PA, PB, OffsetC) + TripleProduct(PA, OffsetB, PC) + TripleProduct(OffsetA, PB, PC),
			TripleProduct(PA, PB, PC)
			};

		// Solve cubic A*w^3 + B*w^2 + C*w + D

		if (FMath::IsNearlyZero(Coeffs[3], double(SMALL_NUMBER)))
		{
			// In this case, the tetrahedron formed above is probably already at zero volume,
			// which means the point is coplanar to the triangle without normal offsets.
			// Just compute the signed distance.
			const FPlane4f TrianglePlane(A, B, C);
			Out[0] = -TrianglePlane.PlaneDot(Point);
			return 1;
		}
		else
		{
			// Search for "near double roots": points where the first derivative is zero and where the function value
			// is close to zero.
			double Extrema[2];
			const double CloseToZeroTolerance = 0.1 * (FMath::Abs(Coeffs[0]) + FMath::Abs(Coeffs[1]) + FMath::Abs(Coeffs[2]) + FMath::Abs(Coeffs[3]));
			const int32 NumExtrema = CubicExtrema(Coeffs[0], Coeffs[1], Coeffs[2], Extrema);

			int32 NumExtremaUsed = 0;
			for (int32 I = 0; I < NumExtrema; ++I)
			{
				const double X = Extrema[I];
				const double FuncValue = Coeffs[0] * X * X * X + Coeffs[1] * X * X + Coeffs[2] * X + Coeffs[3];
				if (FMath::IsNearlyZero(FuncValue, CloseToZeroTolerance))
				{
					Out[NumExtremaUsed++] = Extrema[I];
				}
			}

			// Now find roots of the cubic function
			double Roots[3];
			const int32 NumRoots = CubicSolve(Coeffs, Roots);

			// Combine roots and extrema
			for (int32 I = 0; I < NumRoots; ++I)
			{
				int NextI = NumExtremaUsed + I;
				check(NextI < MaxNumRootsAndExtrema);
				Out[NextI] = Roots[I];
			}

			// Sort by magnitude of roots/extrema to bias towards eventually choosing smaller parameter values
			Algo::Sort(TArrayView<double>(Out, NumExtremaUsed + NumRoots), [](const double& A, const double& B) {
				return FMath::Abs(A) < FMath::Abs(B);
			});

			return NumExtremaUsed + NumRoots;
		}
	}

	FVector4f GetPointBaryAndDistWithNormals(
		const FVector3f& A, const FVector3f& B, const FVector3f& C,
		const FVector3f& InputNA, const FVector3f& InputNB, const FVector3f& InputNC,
		const FVector3f& Point)
	{
		// Input normals are inverse of what they are in the shader code. Flip the sign when computing weights that
		// will be used in the shader code.
		const FVector3f UseNA = -InputNA;
		const FVector3f UseNB = -InputNB;
		const FVector3f UseNC = -InputNC;

		// Adapted from cloth CCD paper [Bridson et al. 2002]
		// First find W such that Point lies in the plane defined by {A+wNA, B+wNB, C+wNC}
		double W[MaxNumRootsAndExtrema];
		const int32 CoplanarityParamCount = CoplanarityParam(A, B, C, UseNA, UseNB, UseNC, Point, W);
		
		if (CoplanarityParamCount == 0)
		{
			// Found no parameter w for which Point \in span{A+wNA, B+wNB, C+wNC}
			// Fall back to using the triangle normal
			return GetPointBaryAndDist(A, B, C, Point);
		}

		const FVector3f ClosestPoint = (FVector3f)FMath::ClosestPointOnTriangleToPoint((FVector)Point, (FVector)A, (FVector)B, (FVector)C);
		const float DistanceToTriangle = FVector3f::Distance(Point, ClosestPoint);
		
		FVector4f BaryAndDist;
		float MinDistanceSq = TNumericLimits<float>::Max();
		bool bAnySolutionFound = false;

		// If the solution gives us barycentric coordinates that lie purely within the triangle,
		// then choose that. Otherwise try to minimize the distance of the projected point to
		// be as close to the triangle as possible.
		for (int32 CoplanarityParamIndex = 0; CoplanarityParamIndex < CoplanarityParamCount; ++CoplanarityParamIndex)
		{
			if (FMath::Abs(W[CoplanarityParamIndex]) > 3.0f * DistanceToTriangle)
			{
				continue;
			}

			// Then find the barycentric coordinates of Point wrt {A+wNA, B+wNB, C+wNC}
			FVector3f AW = A + W[CoplanarityParamIndex] * UseNA;
			FVector3f BW = B + W[CoplanarityParamIndex] * UseNB;
			FVector3f CW = C + W[CoplanarityParamIndex] * UseNC;
		
			FPlane4f TrianglePlane(AW, BW, CW);

			const FVector3f PointOnTriPlane = FVector3f::PointPlaneProject(Point, TrianglePlane);
			
			// check the triangle {A+wNA, B+wNB, C+wNC} is not degenerate
			const FVector TriNorm = ((FVector)BW - (FVector)AW).Cross((FVector)CW - (FVector)AW);
			const double TriNormSizeSquared = TriNorm.SizeSquared();
			if (TriNormSizeSquared <= UE_DOUBLE_SMALL_NUMBER)
			{
				continue;
			}
			 
			const FVector3f BaryCoords = (FVector3f)FMath::ComputeBaryCentric2D((FVector)PointOnTriPlane, (FVector)AW, (FVector)BW, (FVector)CW);

			checkf(!(BaryCoords.X == BaryCoords.Y && BaryCoords.Y == BaryCoords.Z && BaryCoords.Z == 0.0f), 
				TEXT("ComputeBaryCentric2D returned all zeros despite triangle area being non-zero"));

			bAnySolutionFound = true;

			if (BaryCoords.X >= 0.0f && BaryCoords.X <= 1.0f &&
				BaryCoords.Y >= 0.0f && BaryCoords.Y <= 1.0f &&
				BaryCoords.Z >= 0.0f && BaryCoords.Z <= 1.0f)
			{
				BaryAndDist = FVector4f(BaryCoords, (float)W[CoplanarityParamIndex]); // LWC_TODO: precision loss, but CoplanarityParam() would have already lost precision for W
				break;
			}
			
			const float DistSq = FMath::Square(BaryCoords.X - 0.5) +
								 FMath::Square(BaryCoords.Y - 0.5) +
								 FMath::Square(BaryCoords.Z - 0.5);
			
			if (DistSq < MinDistanceSq)
			{
				BaryAndDist = FVector4f(BaryCoords, (float)W[CoplanarityParamIndex]); // LWC_TODO: precision loss, but CoplanarityParam() would have already lost precision for W
				MinDistanceSq = DistSq;
			}
		}


		bool bRequireFallback = false;

		if (bAnySolutionFound)
		{
			const FVector3f ReprojectedPoint =
				BaryAndDist.X * (A + UseNA * BaryAndDist.W) +
				BaryAndDist.Y * (B + UseNB * BaryAndDist.W) +
				BaryAndDist.Z * (C + UseNC * BaryAndDist.W);

			const float Distance = FVector3f::Distance(Point, ReprojectedPoint);

			// Check if the reprojected point is far from the original. If it is, fall back on
			// the old method of computing the bary values.
			// FIXME: Should we test other cage triangles instead? It's possible that
			// GetBestTriangleBaseIndex is not actually picking the /best/ one.
			bRequireFallback = !FMath::IsNearlyZero(Distance, KINDA_SMALL_NUMBER);
		}
		else
		{
			// no viable solution
			bRequireFallback = true;
		}

		if (bRequireFallback)
		{
			return GetPointBaryAndDist(A, B, C, Point);
		}
		
		return BaryAndDist;
	}

	void ComputeVertexContributions(
		TArray<FMeshToMeshVertData>& InOutSkinningData,
		const FPointWeightMap* const InMaxDistances,
		const bool bInSmoothTransition,
		const bool bInUseMultipleInfluences
		)
	{
		if (InMaxDistances && InMaxDistances->Num())
		{
			for (FMeshToMeshVertData& VertData : InOutSkinningData)
			{
				const bool IsStatic0 = InMaxDistances->IsBelowThreshold(VertData.SourceMeshVertIndices[0]);
				const bool IsStatic1 = InMaxDistances->IsBelowThreshold(VertData.SourceMeshVertIndices[1]);
				const bool IsStatic2 = InMaxDistances->IsBelowThreshold(VertData.SourceMeshVertIndices[2]);

				// None of the cloth vertices will move due to max distance constraints.
				if ((IsStatic0 && IsStatic1 && IsStatic2) || (bInUseMultipleInfluences && VertData.Weight == 0.f))
				{
					VertData.SourceMeshVertIndices[3] = 0xFFFF;
				}
				// If all of the vertices are dynamic _or_ if we disallow smooth transition,
				// ensure there's no blending between cloth and skinned mesh and that the cloth
				// mesh dominates.
				else if ((!IsStatic0 && !IsStatic1 && !IsStatic2) || !bInSmoothTransition)
				{
					VertData.SourceMeshVertIndices[3] = 0;
				}
				else
				{
					// Compute how much the vertex actually contributes. A value of 0xFFFF
					// means that it stays static relative to the skinned mesh, a value of 0x0000
					// means that only the cloth simulation contributes. 
					float StaticAlpha = 
						IsStatic0 * VertData.PositionBaryCoordsAndDist.X +
						IsStatic1 * VertData.PositionBaryCoordsAndDist.Y +
						IsStatic2 * VertData.PositionBaryCoordsAndDist.Z;
					StaticAlpha = FMath::Clamp(StaticAlpha, 0.0f, 1.0f);
					
					VertData.SourceMeshVertIndices[3] = static_cast<uint16>(StaticAlpha * 0xFFFF);
				}	
			}
		}
		else
		{
			// Can't determine contribution from the max distance map, so the entire mesh overrides.
			for (FMeshToMeshVertData& VertData : InOutSkinningData)
			{
				VertData.SourceMeshVertIndices[3] = 0;
			}
		}
		
	}

	void FVertexParameterMapper::Map(TConstArrayView<float> Source, TArray<float>& Dest)
	{
		Map(Source, Dest, [](FVector3f Bary, float A, float B, float C)
		{
			return Bary.X * A + Bary.Y * B + Bary.Z * C;
		});
	}

	void FVertexParameterMapper::GenerateEmbeddedPositions(TArray<FVector4>& OutEmbeddedPositions, TArray<int32>& OutSourceIndices)
	{
		const ClothMeshDesc SourceMesh(Mesh1Positions, Mesh1Normals, Mesh1Indices);

		const int32 NumPositions = Mesh0Positions.Num();

		OutEmbeddedPositions.Reset();
		OutEmbeddedPositions.AddUninitialized(NumPositions);

		OutSourceIndices.Reset(NumPositions * 3);

		for(int32 PositionIndex = 0 ; PositionIndex < NumPositions ; ++PositionIndex)
		{
			const FVector3f& Position = Mesh0Positions[PositionIndex];

			const int32 TriBaseIndex = GetBestTriangleBaseIndex(SourceMesh, (FVector)Position);

			const int32 IA = Mesh1Indices[TriBaseIndex];
			const int32 IB = Mesh1Indices[TriBaseIndex + 1];
			const int32 IC = Mesh1Indices[TriBaseIndex + 2];

			const FVector3f& A = Mesh1Positions[IA];
			const FVector3f& B = Mesh1Positions[IB];
			const FVector3f& C = Mesh1Positions[IC];

			const FVector3f& NA = Mesh1Normals[IA];
			const FVector3f& NB = Mesh1Normals[IB];
			const FVector3f& NC = Mesh1Normals[IC];

			OutEmbeddedPositions[PositionIndex] = (FVector4)GetPointBaryAndDistWithNormals(A, B, C, NA, NB, NC, Position);
			OutSourceIndices.Add(IA);
			OutSourceIndices.Add(IB);
			OutSourceIndices.Add(IC);
		}
	}
}

#undef LOCTEXT_NAMESPACE
