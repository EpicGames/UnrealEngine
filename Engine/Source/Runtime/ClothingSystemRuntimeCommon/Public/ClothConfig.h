// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothConfigBase.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothConfig.generated.h"

class UObject;
struct FClothConfig_Legacy;

/** Different mass modes deciding the setup process. */
UENUM()
enum class EClothMassMode : uint8
{
	/** The mass value is used to set the same mass for each particle. */
	UniformMass,
	/** The mass value is used to set the mass of the entire cloth, distributing it to each particle depending on the amount of connected surface area. */
	TotalMass,
	/** The mass value is used to set the density of the cloth, calculating the mass for each particle depending on its connected surface area. */
	Density,
	MaxClothMassMode UMETA(Hidden)
};

/** Common configuration base class. */
UCLASS(Abstract, MinimalAPI)
class UClothConfigCommon : public UClothConfigBase
{
	GENERATED_BODY()
public:
	CLOTHINGSYSTEMRUNTIMECOMMON_API UClothConfigCommon();
	CLOTHINGSYSTEMRUNTIMECOMMON_API virtual ~UClothConfigCommon() override;

	/** Migrate from the legacy FClothConfig structure. */
	virtual void MigrateFrom(const FClothConfig_Legacy&) {}

	/** Migrate from shared configs. */
	virtual void MigrateFrom(const class UClothSharedConfigCommon*) {}

	/**
	 * Migrate to the legacy FClothConfig structure.
	 * Useful for converting configs that are compatible with this legacy structure.
	 * @return true when the migration is possible, false otherwise.
	 */
	virtual bool MigrateTo(FClothConfig_Legacy&) const { return false; }

	//~ Begin UClothConfigBase Interface
	/** Return whether to pre-compute self collision data. */
	virtual bool NeedsSelfCollisionData() const override { return false; }

	/** Return whether to pre-compute inverse masses. */
	virtual bool NeedsInverseMasses() const override { return false; }

	/** Return whether to pre-compute the influences. */
	virtual bool NeedsNumInfluences() const override { return true; }

	/** Return whether to pre-compute the long range attachment tethers. */
	virtual bool NeedsTethers() const override { return false; }

	/** Return the self collision radius to precomute self collision data. */
	virtual float GetSelfCollisionRadius() const override { return 0.f; }

	/** Return whether tethers need to be calculated using geodesic distances instead of eclidean. */
	virtual bool TethersUseGeodesicDistance() const override { return false; }
	//~ End UClothConfigBase Interface
};

/** Common shared configuration base class. */
UCLASS(Abstract, MinimalAPI)
class UClothSharedConfigCommon : public UClothConfigCommon
{
	GENERATED_BODY()
public:
	CLOTHINGSYSTEMRUNTIMECOMMON_API UClothSharedConfigCommon();
	CLOTHINGSYSTEMRUNTIMECOMMON_API virtual ~UClothSharedConfigCommon() override;
};
