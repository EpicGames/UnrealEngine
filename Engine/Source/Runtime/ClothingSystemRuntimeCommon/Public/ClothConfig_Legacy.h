// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"

#include "ClothConfig_Legacy.generated.h"

/**
 * Deprecated, legacy definition kept for backward compatibility only.
 * Use EClothingWindMethodNv instead.
 * Redirected from the now defunct ClothingSystemRuntime module.
 */
UENUM()
enum class EClothingWindMethod_Legacy : uint8
{
	// Use legacy wind mode, where accelerations are modified directly by the simulation
	// with no regard for drag or lift
	Legacy,

	// Use updated wind calculation for NvCloth based solved taking into account
	// drag and lift, this will require those properties to be correctly set in
	// the clothing configuration
	Accurate
};

/**
 * Deprecated, legacy definition kept for backward compatibility only.
 * Use FClothConstraintSetupNv instead.
 * Redirected from the now defunct ClothingSystemRuntime module.
 */
USTRUCT()
struct FClothConstraintSetup_Legacy
{
	GENERATED_BODY()

	FClothConstraintSetup_Legacy()
		: Stiffness(1.0f)
		, StiffnessMultiplier(1.0f)
		, StretchLimit(1.0f)
		, CompressionLimit(1.0f)
	{}

	// How stiff this constraint is, this affects how closely it will follow the desired position
	UPROPERTY()
	float Stiffness;

	// A multiplier affecting the above value
	UPROPERTY()
	float StiffnessMultiplier;

	// The hard limit on how far this constraint can stretch
	UPROPERTY()
	float StretchLimit;

	// The hard limit on how far this constraint can compress
	UPROPERTY()
	float CompressionLimit;
};

/**
 * Deprecated, legacy definition kept for backward compatibility only.
 * Inherit new config class from UClothConfigCommon instead.
 * Redirected from the now defunct ClothingSystemRuntime module.
 */
USTRUCT()
struct FClothConfig_Legacy
{
	GENERATED_BODY()

	FClothConfig_Legacy()
		: WindMethod(EClothingWindMethod_Legacy::Legacy)
		, SelfCollisionRadius(0.0f)
		, SelfCollisionStiffness(0.0f)
		, SelfCollisionCullScale(1.0f)
		, Damping(0.4f)
		, Friction(0.1f)
		, WindDragCoefficient(0.02f/100.0f)
		, WindLiftCoefficient(0.02f/100.0f)
		, LinearDrag(0.2f)
		, AngularDrag(0.2f)
		, LinearInertiaScale(1.0f)
		, AngularInertiaScale(1.0f)
		, CentrifugalInertiaScale(1.0f)
		, SolverFrequency(120.0f)
		, StiffnessFrequency(100.0f)
		, GravityScale(1.0f)
		, GravityOverride(FVector::ZeroVector)
		, bUseGravityOverride(false)
		, TetherStiffness(1.0f)
		, TetherLimit(1.0f)
		, CollisionThickness(1.0f)
		, AnimDriveSpringStiffness(1.0f)
		, AnimDriveDamperStiffness(1.0f)
	{}

	// How wind should be processed, Accurate uses drag and lift to make the cloth react differently, legacy applies similar forces to all clothing without drag and lift (similar to APEX)
	UPROPERTY()
	EClothingWindMethod_Legacy WindMethod;

	// Constraint data for vertical constraints
	UPROPERTY()
	FClothConstraintSetup_Legacy VerticalConstraintConfig;

	// Constraint data for horizontal constraints
	UPROPERTY()
	FClothConstraintSetup_Legacy HorizontalConstraintConfig;

	// Constraint data for bend constraints
	UPROPERTY()
	FClothConstraintSetup_Legacy BendConstraintConfig;

	// Constraint data for shear constraints
	UPROPERTY()
	FClothConstraintSetup_Legacy ShearConstraintConfig;

	// Size of self collision spheres centered on each vert
	UPROPERTY()
	float SelfCollisionRadius;

	// Stiffness of the spring force that will resolve self collisions
	UPROPERTY()
	float SelfCollisionStiffness;

	UPROPERTY()
	float SelfCollisionCullScale;

	// Damping of particle motion per-axis
	UPROPERTY()
	FVector Damping;

	// Friction of the surface when colliding
	UPROPERTY()
	float Friction;

	// Drag coefficient for wind calculations, higher values mean wind has more lateral effect on cloth
	UPROPERTY()
	float WindDragCoefficient;

	// Lift coefficient for wind calculations, higher values make cloth rise easier in wind
	UPROPERTY()
	float WindLiftCoefficient;

	// Drag applied to linear particle movement per-axis
	UPROPERTY()
	FVector LinearDrag;

	// Drag applied to angular particle movement, higher values should limit material bending (per-axis)
	UPROPERTY()
	FVector AngularDrag;

	// Scale for linear particle inertia, how much movement should translate to linear motion (per-axis)
	UPROPERTY()
	FVector LinearInertiaScale;

	// Scale for angular particle inertia, how much movement should translate to angular motion (per-axis)
	UPROPERTY()
	FVector AngularInertiaScale;

	// Scale for centrifugal particle inertia, how much movement should translate to angular motion (per-axis)
	UPROPERTY()
	FVector CentrifugalInertiaScale;

	// Frequency of the position solver, lower values will lead to stretchier, bouncier cloth
	UPROPERTY()
	float SolverFrequency;

	// Frequency for stiffness calculations, lower values will degrade stiffness of constraints
	UPROPERTY()
	float StiffnessFrequency;

	// Scale of gravity effect on particles
	UPROPERTY()
	float GravityScale;

	// Direct gravity override value
	UPROPERTY()
	FVector GravityOverride;

	// Use gravity override value vs gravity scale 
	UPROPERTY()
	bool bUseGravityOverride;

	// Scale for stiffness of particle tethers between each other
	UPROPERTY()
	float TetherStiffness;

	// Scale for the limit of particle tethers (how far they can separate)
	UPROPERTY()
	float TetherLimit;

	// 'Thickness' of the simulated cloth, used to adjust collisions
	UPROPERTY()
	float CollisionThickness;

	// Default spring stiffness for anim drive if an anim drive is in use
	UPROPERTY()
	float AnimDriveSpringStiffness;

	// Default damper stiffness for anim drive if an anim drive is in use
	UPROPERTY()
	float AnimDriveDamperStiffness;
};
