// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothPhysicalMeshData.h"
#include "ClothLODData_Legacy.h"
#include "ClothLODData.generated.h"

/** Common Cloth LOD representation for all clothing assets. */
USTRUCT()
struct FClothLODDataCommon
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For CollisionData
	FClothLODDataCommon() = default;
	FClothLODDataCommon(const FClothLODDataCommon&) = default;
	FClothLODDataCommon(FClothLODDataCommon&&) = default;
	~FClothLODDataCommon() = default;
	FClothLODDataCommon& operator=(const FClothLODDataCommon&) = default;
	FClothLODDataCommon& operator=(FClothLODDataCommon&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Raw phys mesh data
	UPROPERTY(EditAnywhere, Category = SimMesh)
	FClothPhysicalMeshData PhysicalMeshData;

	// Whether to use multiple triangles to interpolate from simulated cloth mesh to render mesh
	UPROPERTY()
	bool bUseMultipleInfluences = false;

	// Radius of the weighting kernel used to interpolate from simulated cloth mesh to render mesh
	UPROPERTY()
	float SkinningKernelRadius = 100.0f;

	// Whether to enable smooth transition from skinned mesh to clothed mesh.
	UPROPERTY()
	bool bSmoothTransition = false;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(all, "This property is no longer supported. Use a Physics Asset instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This property is no longer supported. Use a Physics Asset instead."))
	FClothCollisionData CollisionData_DEPRECATED;

	UE_DEPRECATED(5.5, "This property is no longer supported. Use PointWeightMaps instead.")
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This property is no longer supported. Use PointWeightMaps instead."))
	TArray<FClothParameterMask_Legacy> ParameterMasks_DEPRECATED;

	// Parameter masks defining the physics mesh masked data
	UPROPERTY(EditAnywhere, Category = Masks)
	TArray<FPointWeightMap> PointWeightMaps;

	// Get all available parameter masks for the specified target
	CLOTHINGSYSTEMRUNTIMECOMMON_API void GetParameterMasksForTarget(const uint8 InTarget, TArray<FPointWeightMap*>& OutMasks);
#endif // WITH_EDITORONLY_DATA
#if WITH_EDITOR
	/** Copy \c ParameterMasks to corresponding targets in \c ClothPhysicalMeshData. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void PushWeightsToMesh();
#endif

	// Skinning data for transitioning from a higher detail LOD to this one
	TArray<FMeshToMeshVertData> TransitionUpSkinData;

	// Skinning data for transitioning from a lower detail LOD to this one
	TArray<FMeshToMeshVertData> TransitionDownSkinData;

	// Custom serialize for transition
	CLOTHINGSYSTEMRUNTIMECOMMON_API bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FClothLODDataCommon> : public TStructOpsTypeTraitsBase2<FClothLODDataCommon>
{
	enum
	{
		WithSerializer = true,
	};
};
