// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "PointWeightMap.h"
#include "ClothCollisionData.h"
#include "SkeletalMeshTypes.h"
#include "ClothPhysicalMeshData.h"
#include "ClothLODData_Legacy.generated.h"

class UClothLODDataCommon;
class UClothPhysicalMeshDataBase_Legacy;

/**
 * Deprecated, legacy definition kept for backward compatibility only.
 * Use FPointWeightMap instead.
 * Redirected from the now defunct ClothingSystemRuntime module.
 */
USTRUCT()
struct FClothParameterMask_Legacy
{
	GENERATED_BODY();

	CLOTHINGSYSTEMRUNTIMECOMMON_API FClothParameterMask_Legacy();

	CLOTHINGSYSTEMRUNTIMECOMMON_API void MigrateTo(FPointWeightMap& Weights);

	/** Name of the mask, mainly for users to differentiate */
	UPROPERTY()
	FName MaskName;

	/** The currently targeted parameter for the mask */
	UPROPERTY()
	EWeightMapTargetCommon CurrentTarget;

	/** The maximum value currently in the mask value array */
	UPROPERTY()
	float MaxValue_DEPRECATED;

	/** The maximum value currently in the mask value array */
	UPROPERTY()
	float MinValue_DEPRECATED;

	/** The actual values stored in the mask */
	UPROPERTY()
	TArray<float> Values;

	/** Whether this mask is enabled and able to effect final mesh values */
	UPROPERTY()
	bool bEnabled;
};

/**
 * Deprecated, legacy definition kept for backward compatibility only.
 * Use FClothLODDataCommon instead.
 */
UCLASS(MinimalAPI)
class UClothLODDataCommon_Legacy : public UObject
{
	GENERATED_BODY()
public:
	CLOTHINGSYSTEMRUNTIMECOMMON_API UClothLODDataCommon_Legacy(const FObjectInitializer& Init);
	CLOTHINGSYSTEMRUNTIMECOMMON_API virtual ~UClothLODDataCommon_Legacy();

	// Deprecated, use ClothPhysicalMeshData instead
	UPROPERTY()
	TObjectPtr<UClothPhysicalMeshDataBase_Legacy> PhysicalMeshData_DEPRECATED;

	// Raw phys mesh data
	UPROPERTY()
	FClothPhysicalMeshData ClothPhysicalMeshData;

	// Collision primitive and convex data for clothing collisions
	UPROPERTY()
	FClothCollisionData CollisionData;

#if WITH_EDITORONLY_DATA
	// Parameter masks defining the physics mesh masked data
	UPROPERTY()
	TArray<FPointWeightMap> ParameterMasks;
#endif // WITH_EDITORONLY_DATA

	// Skinning data for transitioning from a higher detail LOD to this one
	TArray<FMeshToMeshVertData> TransitionUpSkinData;

	// Skinning data for transitioning from a lower detail LOD to this one
	TArray<FMeshToMeshVertData> TransitionDownSkinData;

	// Custom serialize for transition
	CLOTHINGSYSTEMRUNTIMECOMMON_API virtual void Serialize(FArchive& Ar) override;

	// Migrate deprecated properties
	CLOTHINGSYSTEMRUNTIMECOMMON_API virtual void PostLoad() override;

	// Migrate this deprecated UObject class to the structure format (called by UClothingAssetCommon::PostLoad())
	CLOTHINGSYSTEMRUNTIMECOMMON_API void MigrateTo(struct FClothLODDataCommon& LodData);
};
