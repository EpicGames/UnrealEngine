// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothTetherData.h"
#include "ClothVertBoneData.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "PointWeightMap.h"
#include "UObject/ObjectMacros.h"

#include "ClothPhysicalMeshData.generated.h"

class FName;
class UClothConfigBase;
class UClothPhysicalMeshDataBase_Legacy;
struct FPointWeightMap;
template <typename T> struct TObjectPtr;

/** Spatial simulation data for a mesh. */
USTRUCT()
struct FClothPhysicalMeshData
{
	GENERATED_BODY()

	/** Construct an empty cloth physical mesh with default common targets. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API FClothPhysicalMeshData();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FClothPhysicalMeshData() = default;
	FClothPhysicalMeshData(const FClothPhysicalMeshData&) = default;
	FClothPhysicalMeshData& operator=(const FClothPhysicalMeshData&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Migrate from same, used to migrate LOD data from the UClothLODDataCommon_Legacy. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void MigrateFrom(FClothPhysicalMeshData& ClothPhysicalMeshData);

	/** Migrate from the legacy physical mesh data class, used to migrate LOD data from the UClothLODDataCommon_Legacy. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void MigrateFrom(UClothPhysicalMeshDataBase_Legacy* ClothPhysicalMeshDataBase);

	/** Reset the default common targets for this cloth physical mesh. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void Reset(const int32 InNumVerts, const int32 InNumIndices);

	/** Clear out any default weight maps and delete any other ones. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void ClearWeightMaps();

	/** Build the self collision indices with the specified radius. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void BuildSelfCollisionData(float SelfCollisionRadius);

	/** Recalculate the node inverse masses. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void CalculateInverseMasses();

	/** Recalculate the number of influences for the bone data. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void CalculateNumInfluences();

	/** Recalculate the long range attachment tethers. */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void CalculateTethers(bool bUseEuclideanDistance, bool bUseGeodesicDistance);

	/** Retrieve whether a vertex weight array has already been registered. */
	template<typename T>
	bool HasWeightMap(const T Target) const { return WeightMaps.Contains((uint32)Target); }

	/** Retrieve a pointer to a registered vertex weight array by unique @param Id, or nullptr if none is found. */
	template<typename T>
	const FPointWeightMap* FindWeightMap(const T Target) const { return WeightMaps.Find((uint32)Target); }

	/** Retrieve a pointer to a registered vertex weight array by unique @param Id, or nullptr if none is found. */
	template<typename T>
	FPointWeightMap* FindWeightMap(const T Target) { return WeightMaps.Find((uint32)Target); }

	/** Retrieve a pointer to a registered vertex weight array by unique @param Id, or add one if it doesn't exist already. */
	template<typename T>
	FPointWeightMap& AddWeightMap(const T Target) { return WeightMaps.Add((uint32)Target); }

	/** Retrieve a pointer to a registered vertex weight array by unique @param Id, or add one if it doesn't exist already. */
	template<typename T>
	FPointWeightMap& FindOrAddWeightMap(const T Target) { return WeightMaps.FindOrAdd((uint32)Target); }

	/** Retrieve a registered vertex weight array by unique @param Id. The array must exists or this function will assert. */
	template<typename T>
	const FPointWeightMap& GetWeightMap(const T Target) const { return WeightMaps[(uint32)Target]; }

	/** Retrieve a registered vertex weight array by unique @param Id. The array must exists or this function will assert. */
	template<typename T>
	FPointWeightMap& GetWeightMap(const T Target) { return WeightMaps[(uint32)Target]; }

	// Positions of each simulation vertex
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TArray<FVector3f> Vertices;

	// Normal at each vertex
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TArray<FVector3f> Normals;

#if WITH_EDITORONLY_DATA
	// Color at each vertex
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TArray<FColor> VertexColors;
#endif // WITH_EDITORONLY_DATA

	// Indices of the simulation mesh triangles
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TArray<uint32> Indices;

	// The weight maps, or masks, used by this mesh, sorted by their target id
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TMap<uint32, FPointWeightMap> WeightMaps;

	// Inverse mass for each vertex in the physical mesh
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TArray<float> InverseMasses;

	// Indices and weights for each vertex, used to skin the mesh to create the reference pose
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TArray<FClothVertBoneData> BoneData;

	// Valid indices to use for self collisions (reduced set of Indices)
	UPROPERTY(EditAnywhere, Category = SimMesh)
	TSet<int32> SelfCollisionVertexSet;

	// Long range attachment tethers, using euclidean (beeline) distance to find the closest attachment
	UPROPERTY(EditAnywhere, Category = SimMesh)
	FClothTetherData EuclideanTethers;

	// Long range attachment tethers, using geodesic (surface) distance to find the closest attachment
	UPROPERTY(EditAnywhere, Category = SimMesh)
	FClothTetherData GeodesicTethers;

	// Maximum number of bone weights of any vetex
	UPROPERTY(EditAnywhere, Category = SimMesh)
	int32 MaxBoneWeights;

	// Number of fixed verts in the simulation mesh (fixed verts are just skinned and do not simulate)
	UPROPERTY(EditAnywhere, Category = SimMesh)
	int32 NumFixedVerts;

	UE_DEPRECATED(5.4, "Use SelfCollisionVertexSet instead.")
	UPROPERTY()
	TArray<uint32> SelfCollisionIndices;
#if WITH_EDITORONLY_DATA
	// Deprecated. Use WeightMaps instead.
	UPROPERTY()
	TArray<float> MaxDistances_DEPRECATED;
	UPROPERTY()
	TArray<float> BackstopDistances_DEPRECATED;
	UPROPERTY()
	TArray<float> BackstopRadiuses_DEPRECATED;
	UPROPERTY()
	TArray<float> AnimDriveMultipliers_DEPRECATED;

#endif // WITH_EDITORONLY_DATA
};
