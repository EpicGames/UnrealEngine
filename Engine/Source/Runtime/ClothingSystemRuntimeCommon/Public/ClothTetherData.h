// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/Tuple.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "ClothTetherData.generated.h"

class FArchive;

/**
 * Long range attachment tether pathfinding based on Dijkstra's algorithm.
 * For more information about the long range attachment, see:
 * https://matthias-research.github.io/pages/publications/sca2012cloth.pdf
 */
USTRUCT()
struct FClothTetherData
{
	GENERATED_BODY()

	/**
	 * Particles whose MaxDistance / TetherEnds mask is strictly less than this are treated as
	 * kinematic for tether sorting. Authoritative source for the legacy "barely allowed to move"
	 * cutoff — referenced by the converter and consumed throughout the cloth simulation pipeline
	 * so the same threshold is applied to legacy assets at convert time and to runtime tether
	 * generation.
	 */
	static constexpr float KinematicDistanceThreshold = 0.1f;

	/** Long range attachment tether start/end/length, sorted in sequential batches of independant tethers. */
	TArray<TArray<TTuple<int32, int32, float>>> Tethers;
	
	/**
	 * Generate the tethers by following the triangle mesh network from the closest kinematic to each dynamic point.
	 * Inside array items can be processed concurrently, but the outside array must be iterated on sequentially.
	 */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void GenerateTethers(
		const TConstArrayView<FVector3f>& Points,  // Reference pose
		const TConstArrayView<uint32>& Indices,  // Triangle mesh
		const TConstArrayView<float>& MaxDistances,  // Mask for sorting the kinematic from the dynamic points
		bool bUseGeodesicDistance);  // Whether to use geodesic (walking along the surface) or euclidean (beeline) distances to find the tethers.
	
	/**
	 * Generate the tethers by following the triangle mesh network from the closest kinematic tether end to each dynamic point.
	 * Inside array items can be processed concurrently, but the outside array must be iterated on sequentially.
	 */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void GenerateTethers(
		const TConstArrayView<FVector3f>& Points,  // Reference pose
		const TConstArrayView<uint32>& Indices,  // Triangle mesh
		const TConstArrayView<float>& MaxDistances,  // Mask for sorting the kinematic from the dynamic points
		const TConstArrayView<float>& TetherEnds,  // Separate mask for specifying tether ends (using same KinematicDistanceThreshold cutoff as MaxDistances). Actual tether end set is the intersection of tether ends and kinematic points.
		bool bUseGeodesicDistance);  // Whether to use geodesic (walking along the surface) or euclidean (beeline) distances to find the tethers.
	
	/**
	 * Generate the tethers by following the triangle mesh network from the closest kinematic to each dynamic point.
	 * Inside array items can be processed concurrently, but the outside array must be iterated on sequentially.
	 */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void GenerateTethers(
		const TConstArrayView<FVector3f>& Points,  // Reference pose
		const TConstArrayView<uint32>& Indices,  // Triangle mesh
		const TSet<int32>& KinematicNodes,  // kinematic indices
		bool bUseGeodesicDistance);  // Whether to use geodesic (walking along the surface) or euclidean (beeline) distances to find the tethers.
	
	/**
	 * Generate the tethers by following the triangle mesh network from the closest kinematic to each dynamic point.
	 * Inside array items can be processed concurrently, but the outside array must be iterated on sequentially.
	 */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void GenerateTethers(
		const TConstArrayView<FVector3f>& Points,  // Reference pose
		const TConstArrayView<uint32>& Indices,  // Triangle mesh
		const TSet<int32>& KinematicNodes,  // all kinematic indices
		const TSet<int32>& FixedEndNodes,  // subset of kinematic indices used for fixed end.
		const TSet<int32>& DynamicEndNodes,  // subset of dynamic indices to create tethers for.
		bool bUseGeodesicDistance);  // Whether to use geodesic (walking along the surface) or euclidean (beeline) distances to find the tethers.

	/**
	 * Generate the tether batches from existing tether data.
	 * PerDynamicNodeTethers is an array where Index = DynamicIndex, and Value is an array of TPair<float, int32>  = {Distance, KinematicIndex}
	 */
	CLOTHINGSYSTEMRUNTIMECOMMON_API void GenerateTethers(
		TArray<TArray<TPair<float, int32>>>&& PerDynamicNodeTethers);

	/** Custom serializer, since neiher an array of array nor a tuple can be set as a UPROPERTY. */ 
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FClothTetherData> : public TStructOpsTypeTraitsBase2<FClothTetherData>
{
	enum
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
