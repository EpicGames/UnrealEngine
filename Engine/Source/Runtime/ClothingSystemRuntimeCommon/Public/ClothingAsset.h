// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothingAssetBase.h"
#include "ClothConfigBase.h"
#include "ClothLODData.h"
#include "ClothConfig_Legacy.h"
#include "ClothLODData_Legacy.h"
#include "ClothingSimulationInteractor.h"
#include "Algo/AnyOf.h"
#include "ClothingAsset.generated.h"

#define UE_API CLOTHINGSYSTEMRUNTIMECOMMON_API

class UPhysicsAsset;
struct FSkelMeshSection;

namespace ClothingAssetUtils
{
	/**
	 * Helper struct to hold binding information on a clothing asset, used to 
	 * enumerate all of the bindings on a skeletal mesh with 
	 * \c GetAllMeshClothingAssetBindings() below.
	 */
	struct FClothingAssetMeshBinding
	{
		UClothingAssetBase* Asset;
		int32 LODIndex;
		int32 SectionIndex;
		int32 AssetInternalLodIndex;
	};

#if WITH_EDITOR
	/**
	 * Unbind Clothing Data from the specified Skeletal Mesh LOD sections.
	 * If MeshLodIndex is INDEX_NONE, the Clothing Data will get unbound from all LODs and all sections (ignoring SectionIndex in this case).
	 * If SectionIndex is INDEX_NONE, the Clothing Data will get unbound from all sections of the specified LOD.
	 * @param Asset The clothing data to unbind.
	 * @param SkeletalMesh The Skeletal Mesh to unbind the Clothing Data from.
	 * @param MeshLodIndex Mesh LOD to remove this asset from (could still be bound to other LODs), or INDEX_NONE to unbind all LODs.
	 * @param SectionIndex The Skeletal Mesh section to unbind, or INDEX_NONE to unbind all sections.
	 */
	UE_API void UnbindFromSkeletalMesh(const UClothingAssetBase& Asset, USkeletalMesh& SkeletalMesh, const int32 MeshLodIndex = INDEX_NONE, const int32 SectionIndex = INDEX_NONE);

	/**
	 * Return whether there are any cloth bindings whithin the skeletal mesh model.
	 * Uses the cloth mapping data as the source predicate to match the behavior of GetAllMeshClothingAssetBindings().
	 * @param SkeletalMesh The skeletal mesh to check the cloth binding from.
	 */
	UE_API bool HasAnyMeshClothingAssetBindings(const USkeletalMesh* SkeletalMesh);

	/**
	 * Return whether there are any cloth bindings whithin the skeletal mesh model LOD.
	 * Uses the cloth mapping data as the source predicate to match the behavior of GetAllLodMeshClothingAssetBindings().
	 * @param SkeletalMesh The skeletal mesh to check the cloth binding from.
	 * @param LodIndex The specified LOD.
	 */
	UE_API bool HasAnyLodMeshClothingAssetBindings(const USkeletalMesh* SkeletalMesh, const int32 LodIndex);

	/**
	 * Given a skeletal mesh model, find all of the currently bound clothing assets and their binding information.
	 * @param SkeletalMesh The skeletal mesh to extract binding from.
	 * @param OutBindings The list of bindings to write to.
	 */
	UE_API void GetAllMeshClothingAssetBindings(const USkeletalMesh* SkeletalMesh, TArray<FClothingAssetMeshBinding>& OutBindings);

	/**
	 * Given a skeletal mesh model, find all of the currently bound clothing assets and their binding information.
	 * @param SkeletalMesh The skeletal mesh to extract binding from.
	 * @param OutBindings The list of bindings to write to.
	 * @param LodIndex The specified LOD.
	 */
	UE_API void GetAllLodMeshClothingAssetBindings(
		const USkeletalMesh* SkeletalMesh,
		TArray<FClothingAssetMeshBinding>& OutBindings,
		int32 LodIndex);

	/**
	 * Clear all deformer section that relies on the specified cloth sim data bound to a LOD section.
	 * @param SkeletalMesh The skeletal mesh to clear.
	 * @param UpdatedLodIndex The recently updated Skeletal Mesh LOD index from which the bias mapping needs to be cleared.
	 * @param SectionIndex The section to clear.
	 */
	UE_API void ClearLODBiasMappings(USkeletalMesh& SkeletalMesh, const int32 UpdatedLodIndex, const int32 SectionIndex);

	/**
	 * Add extra cloth deformer mappings to cope with a different raytracing LOD than the one currently rendered.
	 * @param Asset The Clothing Asset to use to generate the mapping of the section for all LODs above the updated LOD up to the max LOD.
	 * @param SkeletalMesh The Skeletal Mesh for which to calculate the render deformer mapping
	 * @param UpdatedLODIndex The Skeletal Mesh LOD that has just been updated. Only LOD from above this one will receive new bias mapping if they exist.
	 * @param SectionIndex The Skeletal Mesh section index to update.
	 * @param bSmoothTransition Whether to create a smooth transition area between skinned render vertices and deformed render vertices using the vertex mapping blend weight.
	 * @param bUseMultipleInfluences Whether to use multiple simulation mesh triangles to set the deformer influences of the render vertices. 
	 * @param SkinningKernelRadius The radius from which to search for the closest simulation mesh triangles.
	 */
	UE_API void UpdateLODBiasMappings(
		const UClothingAssetBase& Asset,
		const USkeletalMesh* SkeletalMesh,
		const int32 UpdatedLODIndex,
		const int32 SectionIndex, 
		const bool bSmoothTransition = true,
		const bool bUseMultipleInfluences = false,
		const float SkinningKernelRadius = 20.f);

	UE_DEPRECATED(5.7, "Unused low level function, will be removed or re-added as a member to FSkelMeshSection.")
	UE_API void ClearSectionClothingData(FSkelMeshSection& Section);
#endif
}

/**
 * Custom data wrapper for clothing assets.
 */
class UE_DEPRECATED(5.0, "Redundant class.") UClothingAssetCustomData;
UCLASS(Abstract, MinimalAPI)
class UClothingAssetCustomData : public UObject
{
	GENERATED_BODY()
};

/**
 * Common flags used by InvalidateCachedData.
 */
enum class EClothingCachedDataFlagsCommon : uint8
{
	None = 0,
	InverseMasses = 1 << 0,
	NumInfluences = 1 << 1,
	SelfCollisionData = 1 << 2,
	Tethers = 1 << 3,
	All = 0xFF
};
ENUM_CLASS_FLAGS(EClothingCachedDataFlagsCommon);

/**
 * Implementation of non-solver specific, but common Engine related functionality.
 *
 * Solver specific implementations may wish to override this class to construct
 * their own default instances of child classes, such as \c ClothSimConfig and 
 * \c CustomData, as well as override the \c AddNewLod() factory to build their 
 * own implementation of \c UClothLODDataBase.
 */
UCLASS(hidecategories = Object, BlueprintType, MinimalAPI)
class UClothingAssetCommon : public UClothingAssetBase
{
	GENERATED_BODY()
public:

	UE_API UClothingAssetCommon(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR

	/**
	 * Create weights for skinning the render mesh to our simulation mesh, and 
	 * weights to drive our sim mesh from the skeleton.
	 */
	UE_API virtual bool BindToSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex, const int32 InAssetLodIndex) override;

	/**
	 * Unbind Clothing Data from the specified Skeletal Mesh LOD sections.
	 * If MeshLodIndex is INDEX_NONE, the Clothing Data will get unbound from all LODs and all sections (ignoring SectionIndex in this case).
	 * If SectionIndex is INDEX_NONE, the Clothing Data will get unbound from all sections of the specified LOD.
	 * @param InSkelMesh The Skeletal Mesh to unbind the Clothing Data from.
	 * @param InMeshLodIndex Mesh LOD to remove this asset from (could still be bound to other LODs), or INDEX_NONE to unbind all LODs.
	 * @param InSectionIndex The Skeletal Mesh section to unbind, or INDEX_NONE to unbind all sections.
	 */
	UE_API virtual void UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex) override;
	
	UE_DEPRECATED(5.8, "Use UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex) instead.")
	virtual void UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh) override
	{
		UnbindFromSkeletalMesh(InSkelMesh, INDEX_NONE, INDEX_NONE);
	}

	UE_DEPRECATED(5.8, "Use UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex) instead.")
	virtual void UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex) override
	{
		UnbindFromSkeletalMesh(InSkelMesh, InMeshLodIndex, INDEX_NONE);
	}

	/**
	 * Update all extra LOD deformer mappings.
	 * This should be called whenever the raytracing LOD bias has changed.
	 */
	UE_API virtual void UpdateAllLODBiasMappings(USkeletalMesh* SkeletalMesh) override;

	/**
	 * Return the simulation mesh.
	 * Used to calculate the LOD bias deformer mappings.
	 * @param OutPositions The simulation mesh positions.
	 * @param OutIndices The simulation mesh triangle indices.
	 * @param OutMaxDistances The simulation mesh max distance attributes (0 for kinematic, >0 for dynamic). must match the number of positions.
	 */
	UE_API virtual void GetSimulationMesh(const int32 InMeshLodIndex, TArray<FVector3f>& OutPositions, TArray<uint32>& OutIndices, TArray<float>& OutMaxDistances) const override;

	/** 
	 * Callback envoked after weights have been edited.
	 * Calls \c PushWeightsToMesh() on each \c ClothLodData, and invalidates cached data. 
	 * Optionaly recalculate the owner's sections fixed vertex data based on this asset masks, and invalidate the DDC for this asset.
	 */
	UE_API void ApplyParameterMasks(bool bUpdateFixedVertData = false, bool bInvalidateDerivedDataCache = true);

	/**
	 * Builds the LOD transition data.
	 * When we transition between LODs we skin the incoming mesh to the outgoing mesh
	 * in exactly the same way the render mesh is skinned to create a smooth swap
	 */
	UE_API void BuildLodTransitionData();

	//~ Begin UObject interface
	/**
	 * Stop any simulation from using this asset.
	 */
	UE_API virtual void PreEditUndo() override;

	/**
	 * Restart simulation using this asset after undo change.
	 */
	UE_API virtual void PostEditUndo() override;
	//~ End UObject interface
#endif // WITH_EDITOR

	/**
	 * Rebuilds the \c UsedBoneIndices array from looking up the entries in the
	 * \c UsedBoneNames array, in the \p InSkelMesh's reference skeleton.
	 */
	UE_API virtual void RefreshBoneMapping(USkeletalMesh* InSkelMesh) override;

	/** Calculates the preferred root bone for the simulation. */
	UE_API void CalculateReferenceBoneIndex();

	/** Returns \c true if \p InLodIndex is a valid LOD id (index into \c ClothLodData). */
	UE_API virtual bool IsValidLod(int32 InLodIndex) const override;

	/** Returns the number of valid LOD's (length of the \c ClothLodData array). */
	UE_API virtual int32 GetNumLods() const override;

#if WITH_EDITORONLY_DATA
	/**
	 * Called on the clothing asset when the base data (physical mesh, config etc.)
	 * has changed, so any intermediate generated data can be regenerated.
	 */
	virtual void InvalidateAllCachedData() override { InvalidateFlaggedCachedData(EClothingCachedDataFlagsCommon::All); }

	/**
	 * Called on the clothing asset when the base data (physical mesh, config etc.)
	 * has changed, so any intermediate generated data can be regenerated.
	 */
	UE_API void InvalidateFlaggedCachedData(EClothingCachedDataFlagsCommon Flags);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/** Add a new LOD class instance. */
	UE_API virtual int32 AddNewLod() override;

	/** Remove the last clothing LOD (and unbind it). */
	UE_API void RemoveLastLod();

	/* Called after changes in any of the asset properties. */
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& ChainEvent) override;
#endif // WITH_EDITOR

	/** Return a const cloth config pointer of the desired cloth config type, or nullptr if there isn't any suitable. */
	template<typename ClothConfigType, typename = typename TEnableIf<TIsDerivedFrom<ClothConfigType, UClothConfigBase>::IsDerived>::Type>
	const ClothConfigType* GetClothConfig() const
	{
		auto const* const ClothConfig = ClothConfigs.Find(ClothConfigType::StaticClass()->GetFName());
		return ClothConfig ? ExactCast<ClothConfigType>(*ClothConfig) : nullptr;
	}

	/** Return a cloth config pointer of the desired cloth config type, or nullptr if there isn't any suitable. */
	template<typename ClothConfigType, typename = typename TEnableIf<TIsDerivedFrom<ClothConfigType, UClothConfigBase>::IsDerived>::Type>
	ClothConfigType* GetClothConfig()
	{
		auto const* const ClothConfig = ClothConfigs.Find(ClothConfigType::StaticClass()->GetFName());
		return ClothConfig ? ExactCast<ClothConfigType>(*ClothConfig) : nullptr;
	}

	/** Migrate deprecated objects. */
	UE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	/** Serialize deprecated objects. */
	UE_API virtual void Serialize(FArchive& Ar) override;

	/** Propagate the shared simulation configs between assets. Called after all cloth assets sharing the same simulation are loaded. */
	UE_API virtual void PostUpdateAllAssets() override;

	// The physics asset to extract collisions from when building a simulation.
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Clothing")
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	// Simulation specific cloth parameters. 
	// Use GetClothConfig() to retrieve the correct parameters/config type for the desired cloth simulation system.
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, EditFixedSize, Instanced, Category = "Skeletal Mesh Clothing")
	TMap<FName, TObjectPtr<UClothConfigBase>> ClothConfigs;

#if WITH_EDITORONLY_DATA
	// Shared by all cloth instances in a skeletal mesh
	// Only supported with Chaos Cloth for now
	// This may not be editable on unused cloth assets
	UPROPERTY()
	TObjectPtr<UClothConfigBase> ClothSharedSimConfig_DEPRECATED;

	// Parameters for how the NVcloth behaves.
	// These will have no effect on Chaos cloth
	UPROPERTY()
	TObjectPtr<UClothConfigBase> ClothSimConfig_DEPRECATED;

	// Parameters for how Chaos cloth behaves 
	// These will not affect NVcloth
	// For now, we have two configuration parameters so that we can switch between chaos and
	// non chaos at will without losing the original NVcloth data
	UPROPERTY()
	TObjectPtr<UClothConfigBase> ChaosClothSimConfig_DEPRECATED;

	// Deprecated. Use LodData instead
	UPROPERTY()
	TArray<TObjectPtr<UClothLODDataCommon_Legacy>> ClothLodData_DEPRECATED;
#endif

	// The actual asset data, listed by LOD.
	UPROPERTY()
	TArray<FClothLODDataCommon> LodData;

	// Tracks which clothing LOD each skel mesh LOD corresponds to (LodMap[SkelLod]=ClothingLod).
	UPROPERTY()
	TArray<int32> LodMap;

	// List of bones this asset uses inside its parent mesh.
	UPROPERTY()
	TArray<FName> UsedBoneNames;

	// List of the indices for the bones in UsedBoneNames, used for remapping.
	UPROPERTY()
	TArray<int32> UsedBoneIndices;

	// Bone to treat as the root of the simulation space.
	UPROPERTY()
	int32 ReferenceBoneIndex;

#if WITH_EDITORONLY_DATA
	/** 
	 * Deprecated property for transitioning the \c FClothConfig struct to the 
	 * \c UClothConfigBase array, in a new property called \c ClothConfigs.
	 */
	UPROPERTY()
	FClothConfig_Legacy ClothConfig_DEPRECATED;
#endif

private:

	// Add or replace a new cloth config of the specified type.
	template<typename ClothConfigType, typename = typename TEnableIf<TIsDerivedFrom<ClothConfigType, UClothConfigBase>::IsDerived>::Type>
	void SetClothConfig(ClothConfigType* ClothConfig)
	{
		check(ClothConfig);
		ClothConfigs.Add(ClothConfig->GetClass()->GetFName(), static_cast<UClothConfigBase*>(ClothConfig));
	}

	// Create and add any missing cloth configs.
	// If a config from a different factory exists already, the newly
	// created config will attempt to initialize its parameters from it.
	// Return true when at least one config has been added, false otherwise.
	bool AddClothConfigs();

	// Propagate the shared simulation configs between assets.
	// Also migrate all deprecated shared parameters which have been moved to the per cloth configs if required.
	// Called after a cloth asset is created or loaded.
	void PropagateSharedConfigs(bool bMigrateSharedConfigToConfig=false);

	// Return true when any one of the cloth configs fullfill the predicate.
	// Used to select which type of data to cache.
	template<typename PredicateType>
	bool AnyOfClothConfigs(PredicateType Predicate) const
	{
		return Algo::AnyOf(ClothConfigs, [&Predicate](const TPair<FName, TObjectPtr<UClothConfigBase>>& ClothConfig)
			{
				return Predicate(*ClothConfig.Value);
			});
	}

#if WITH_EDITOR
	// Return all LODs' stats.
	virtual TArray<TPair<FText, FText>> GetStats() const override;

	// Helper functions used in PostPropertyChangeCb
	void ReregisterComponentsUsingClothing();
	void ForEachInteractorUsingClothing(TFunction<void (UClothingSimulationInteractor*)> Func);
#endif // WITH_EDITOR
};

#undef UE_API
