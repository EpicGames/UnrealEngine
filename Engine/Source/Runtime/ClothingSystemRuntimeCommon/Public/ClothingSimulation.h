// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothingSimulationInterface.h"
#include "ClothingSimulationCacheData.h"
#include "Containers/Array.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Stats/Stats.h"

#define UE_API CLOTHINGSYSTEMRUNTIMECOMMON_API
class USkeletalMeshComponent;

// Common simulation stats
DECLARE_CYCLE_STAT_EXTERN(TEXT("Compute Clothing Normals"), STAT_ClothComputeNormals, STATGROUP_Physics, UE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Internal Solve"), STAT_ClothInternalSolve, STATGROUP_Physics, UE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Collisions"), STAT_ClothUpdateCollisions, STATGROUP_Physics, UE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Skin Physics Mesh"), STAT_ClothSkinPhysMesh, STATGROUP_Physics, UE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Fill Context"), STAT_ClothFillContext, STATGROUP_Physics, UE_API);

/** Base simulation data that just about every simulation would need. */
class FClothingSimulationContextCommon : public IClothingSimulationContext
{
public:
	UE_API FClothingSimulationContextCommon();
	FClothingSimulationContextCommon(const FClothingSimulationContextCommon&) = default;
	FClothingSimulationContextCommon& operator=(const FClothingSimulationContextCommon&) = default;

	UE_API virtual ~FClothingSimulationContextCommon() override;

	// Fill this context using the given skeletal mesh component
	UE_API virtual void Fill(const USkeletalMeshComponent* InComponent, float InDeltaSeconds, float InMaxPhysicsDelta, bool bIsInitialization);

protected:
	// Default fill behavior as expected to be used by every simulation
	UE_API virtual void FillBoneTransforms(const USkeletalMeshComponent* InComponent);
	UE_API virtual void FillRefToLocals(const USkeletalMeshComponent* InComponent, bool bIsInitialization);
	UE_API virtual void FillComponentToWorld(const USkeletalMeshComponent* InComponent);
	UE_API virtual void FillWorldGravity(const USkeletalMeshComponent* InComponent);
	UE_API virtual void FillWindVelocity(const USkeletalMeshComponent* InComponent);
	UE_API virtual void FillDeltaSeconds(float InDeltaSeconds, float InMaxPhysicsDelta);
	UE_API virtual void FillTeleportMode(const USkeletalMeshComponent* InComponent, float InDeltaSeconds, float InMaxPhysicsDelta);
	UE_API virtual void FillMaxDistanceScale(const USkeletalMeshComponent* InComponent);
	UE_API virtual void FillSolverGeometryScale(const USkeletalMeshComponent* InComponent);

public:
	// Component space bone transforms of the owning component
	TArray<FTransform> BoneTransforms;

	// Ref to local matrices from the owning component (for skinning fixed verts)
	TArray<FMatrix44f> RefToLocals;

	// Component to world transform of the owning component
	FTransform ComponentToWorld;

	// Gravity extracted from the world
	FVector WorldGravity;

	// Wind velocity at the component location
	FVector WindVelocity;

	// Wind adaption, a measure of how quickly to adapt to the wind speed
	// when using the legacy wind calculation mode
	float WindAdaption;

	// Delta for this tick
	float DeltaSeconds;

	// Velocity scale to compensate input velocities in case the MaxPhysicsDelta kicks in
	float VelocityScale;

	// Whether and how we should teleport the simulation this tick
	EClothingTeleportMode TeleportMode;

	// Scale for the max distance constraints of the simulation mesh
	float MaxDistanceScale;

	// Scale applied to all incoming geometry
	float SolverGeometryScale = 1.f;

	// The predicted LOD of the skeletal mesh component running the simulation
	int32 PredictedLod;
	
	// Data read from the cache.
	FClothingSimulationCacheData CacheData;
};

// Base simulation to fill in common data for the base context
class UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.") FClothingSimulationCommon
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: public IClothingSimulation  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	UE_API FClothingSimulationCommon();
	UE_API virtual ~FClothingSimulationCommon();

protected:
	/** Fills in the base data for a clothing simulation */
	UE_API virtual void FillContextAndPrepareTick(const USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly) override;

protected:
	/** Maximum physics time, incoming deltas will be clamped down to this value on long frames */
	float MaxPhysicsDelta;
};

#undef UE_API
