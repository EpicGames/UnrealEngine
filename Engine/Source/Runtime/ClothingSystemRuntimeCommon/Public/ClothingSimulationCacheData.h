// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Math/Transform.h"
#include "Math/Vector.h"

struct FClothingSimulationCacheData
{
	// Solver space cached positions for the kinematics targets. (Sparse--use CacheIndices to map to full)
	TArray<FVector> CachedPositions;

	// Solver space cached velocities for the kinematics targets. (Sparse--use CacheIndices to map to full)
	TArray<FVector> CachedVelocities;

	// Indices used to map from CachedPositions/Velocities to solver indices
	TArray<int32> CacheIndices;

	// Cached ReferenceSpaceTransform per cloth. Key is Cloth GroupId
	TMap<int32, FTransform> CachedReferenceSpaceTransforms;

	FClothingSimulationCacheData() = default;
	FClothingSimulationCacheData(const FClothingSimulationCacheData& Other) = default;
	FClothingSimulationCacheData(FClothingSimulationCacheData&& Other)
		: CachedPositions(MoveTemp(Other.CachedPositions))
		, CachedVelocities(MoveTemp(Other.CachedVelocities))
		, CacheIndices(MoveTemp(Other.CacheIndices))
		, CachedReferenceSpaceTransforms(MoveTemp(Other.CachedReferenceSpaceTransforms))
	{}

	FClothingSimulationCacheData& operator=(const FClothingSimulationCacheData& Other) = default;
	FClothingSimulationCacheData& operator=(FClothingSimulationCacheData&& Other)
	{
		CachedPositions = MoveTemp(Other.CachedPositions);
		CachedVelocities = MoveTemp(Other.CachedVelocities);
		CacheIndices = MoveTemp(Other.CacheIndices);
		CachedReferenceSpaceTransforms = MoveTemp(Other.CachedReferenceSpaceTransforms);
		return *this;
	}

	void Reset()
	{
		CachedPositions.Reset();
		CachedVelocities.Reset();
		CacheIndices.Reset();
		CachedReferenceSpaceTransforms.Reset();
	}

	bool HasData() const
	{
		checkSlow(CachedPositions.Num() == CacheIndices.Num());
		return CacheIndices.Num() > 0;
	}
};