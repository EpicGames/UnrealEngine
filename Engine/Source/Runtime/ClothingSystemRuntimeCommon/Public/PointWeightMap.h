// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <type_traits>

#include "UObject/ObjectMacros.h"
#include "PointWeightMap.generated.h"

/** The possible targets for a physical mesh point weight map. */
UENUM()
enum class EWeightMapTargetCommon : uint8
{
	None = 0,            // None, should always be zero
	MaxDistance,         // The distance that each vertex can move away from its reference (skinned) position
	BackstopDistance,    // Distance along the plane of the surface that the particles can travel (separation constraint)
	BackstopRadius,      // Radius of movement to allow for backstop movement
	AnimDriveStiffness,  // Strength of anim drive per-particle (spring driving particle back to skinned location
	AnimDriveDamping_DEPRECATED UMETA(Hidden),  // Chaos onlyweightmap, deprecated from the common declaration
	FirstUserTarget,
	// Users can add additional targets between this range.
	LastUserTarget = 200,
	// Add additional common targets after here.
	TetherEndsMask
};

/** 
 * A mask is simply some storage for a physical mesh parameter painted onto clothing.
 * Used in the editor for users to paint onto and then target to a parameter, which
 * is then later applied to a phys mesh
 */
USTRUCT()
struct FPointWeightMap
{
	GENERATED_BODY();

	FPointWeightMap()
#if WITH_EDITORONLY_DATA
		: Name(NAME_None)
		, CurrentTarget((uint8)EWeightMapTargetCommon::None)
		, bEnabled(false)
#endif
	{}

	explicit FPointWeightMap(int32 NumPoints, float Value = 0.f)
#if WITH_EDITORONLY_DATA
		: Name(NAME_None)
		, CurrentTarget((uint8)EWeightMapTargetCommon::None)
		, bEnabled(false)
#endif
	{
		Values.Init(Value, NumPoints);
	}

	explicit FPointWeightMap(const TConstArrayView<float>& InValues)
		: Values(InValues)
#if WITH_EDITORONLY_DATA
		, Name(NAME_None)
		, CurrentTarget((uint8)EWeightMapTargetCommon::None)
		, bEnabled(false)
#endif
	{}

	FPointWeightMap(const TConstArrayView<float>& InValues, float Offset, float Scale)
#if WITH_EDITORONLY_DATA
		: Name(NAME_None)
		, CurrentTarget((uint8)EWeightMapTargetCommon::None)
		, bEnabled(false)
#endif
	{
		const int32 NumPoints = InValues.Num();
		Values.SetNumUninitialized(NumPoints);
		for (int32 Index = 0; Index < NumPoints; ++Index)
		{
			Values[Index] = Offset + Scale * InValues[Index];
		}
	}

	~FPointWeightMap() = default;

	/**
	 * Reset this map to the specified number of points, and set all the values to zero.
	 * @param NumPoints the number of points to initialize the map with.
	 */
	void Initialize(const int32 NumPoints)
	{
		Values.Init(0.f, NumPoints);
#if WITH_EDITORONLY_DATA
		CurrentTarget = (uint8)EWeightMapTargetCommon::None;
		bEnabled = false;
#endif
	}

	/**
	 * Initialize a weight map from another weight map while enabling and setting a new target.
	 * @param Source the source weight map to copy the values from.
	 * @param Target the new weight map target. */
	template <
		typename T
		UE_REQUIRES(std::is_enum_v<T> || std::is_arithmetic_v<T>)
	>
	void Initialize(const FPointWeightMap& Source, T Target)
	{
		Values = Source.Values;
#if WITH_EDITORONLY_DATA
		CurrentTarget = (uint8)Target;
		bEnabled = true;
#endif
	}

	/** Empty this map of any values. */
	void Empty()
	{ Values.Empty(); }

	/** Return the number of values in this map. */
	int32 Num() const
	{ return Values.Num(); }

	/**
	 * Return the current float value for the requested point.
	 * @param Index the value/point index to retrieve, must be within range or it will assert.
	 */
	const float& operator[](int32 Index) const
	{ return Values[Index]; }

	/**
	 * Return the current float value for the requested point.
	 * @param Index the value/point index to retrieve, must be within range or it will assert.
	 */
	float& operator[](int32 Index)
	{ return Values[Index]; }

	/** 
	 * Get a value from the map, or return 0 if the index is out of bounds.
	 * @param Index the value/point index to retrieve
	 */
	float GetValue(int32 Index) const
	{ return Values.IsValidIndex(Index) ? Values[Index] : 0.f; }

	/** 
	 * Set a value in the map checking first whether the index is within bounds.
	 * @param Index the value/point index to set
	 * @param Value the value to set
	 */
	void SetValue(int32 Index, float Value)
	{ if (Values.IsValidIndex(Index)) Values[Index] = Value; }

	/**
	 * Return whether the specified point weight is below (or equal) to the specified threshold.
	 * @param Index the value/point index to retrieve
	 * @param Threshold, the value threshold to test against for.
	 */
	bool IsBelowThreshold(const int32 Index, const float Threshold=0.1f) const
	{ return Values.IsValidIndex(Index) && Values[Index] <= Threshold; }

	/** Return whether at least one of the specified triangle points has weight below (or equal) to the specified @param Threshold. */
	bool AreAnyBelowThreshold(const int32 Index0, const int32 Index1, const int32 Index2, const float Threshold=0.1f) const
	{ return IsBelowThreshold(Index0, Threshold) || IsBelowThreshold(Index1, Threshold) || IsBelowThreshold(Index2, Threshold); }

	/** Return whether all of the specified triangle points have weight below (or equal) to the specified @param Threshold. */
	bool AreAllBelowThreshold(const int32 Index0, const int32 Index1, const int32 Index2, const float Threshold=0.1f) const
	{ return IsBelowThreshold(Index0, Threshold) && IsBelowThreshold(Index1, Threshold) && IsBelowThreshold(Index2, Threshold); }

	/** Return whether all points' values are zero. */
	bool IsZeroed() const
	{ return !Values.FindByPredicate([](const float& Value) { return Value != 0.f; }); }

	/** Calculates Min/Max values based on values. */
	void CalcRanges(float& MinValue, float& MaxValue)
	{ MinValue = FMath::Min(Values); MaxValue = FMath::Max(Values); }

	/** The actual values stored in the mask */
	UPROPERTY()
	TArray<float> Values;

#if WITH_EDITORONLY_DATA
	/** Name of the mask, mainly for users to differentiate */
	UPROPERTY()
	FName Name;

	/** The currently targeted parameter for the mask, @seealso EWeightMapTargetCommon */
	UPROPERTY()
	uint8 CurrentTarget;

	/** Whether this mask is enabled and able to effect final mesh values */
	UPROPERTY()
	bool bEnabled;
#endif
};
