// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkeletalMeshTypes.h"
#include "Chaos/AABBTree.h"

DECLARE_LOG_CATEGORY_EXTERN(LogClothingMeshUtils, Log, All);

struct FClothPhysicalMeshData;
struct FPointWeightMap;

namespace ClothingMeshUtils
{
	struct FMeshToMeshFilterSet
	{
		TSet<int32> SourceTriangles;  // Set of triangle index in the source ClothMeshDesc
		TSet<int32> TargetVertices;  // Set of vertex index in the target ClothMeshDesc
	};

	class ClothMeshDesc
	{
	public:
		ClothMeshDesc(TConstArrayView<FVector3f> InPositions, TConstArrayView<FVector3f> InNormals, TConstArrayView<uint32> InIndices)
			: Positions(InPositions)
			, Normals(InNormals)
			, Indices(InIndices)
			, bHasValidBVH(false)
		{
		}

		ClothMeshDesc(TConstArrayView<FVector3f> InPositions, TConstArrayView<FVector3f> InNormals, TConstArrayView<FVector3f> InTangents, TConstArrayView<uint32> InIndices)
			: Positions(InPositions)
			, Normals(InNormals)
			, Tangents(InTangents)
			, Indices(InIndices)
			, bHasValidBVH(false)
		{
		}

		/* Construct a mesh descriptor with re-calculated (averaged) normals to match the simulation output. */
		ClothMeshDesc(TConstArrayView<FVector3f> InPositions, TConstArrayView<uint32> InIndices)
			: Positions(InPositions)
			, Indices(InIndices)
			, bHasValidBVH(false)
		{
			ComputeAveragedNormals();
			Normals = TConstArrayView<FVector3f>(AveragedNormals);
		}

		const TConstArrayView<FVector3f>& GetPositions() const { return Positions; }
		const TConstArrayView<FVector3f>& GetNormals() const { return Normals; }
		const TConstArrayView<FVector3f>& GetTangents() const { return Tangents; }
		const TConstArrayView<uint32>& GetIndices() const { return Indices; }

		/** Return whether the mesh descriptor has a valid number of normals and triangle indices. */
		bool HasValidMesh() const { return Positions.Num() == Normals.Num() && Indices.Num() % 3 == 0; }

		/** Return whether this descriptor has been initialized with tangent information. */
		bool HasTangents() const { return Positions.Num() == Tangents.Num(); }

		/** Return whether this descriptor has been initialized with the normals generated from the averaged triangle normals. */
		bool HasAveragedNormals() const { return Normals.Num() == AveragedNormals.Num(); }

		/** Find the distance to the specified triangle from a point. */
		CLOTHINGSYSTEMRUNTIMECOMMON_API float DistanceToTriangle(const FVector& Position, int32 TriangleBaseIndex) const;

		/** Return the current list of filtered triangles. */
		TArray<int32>& GetFilteredTriangles() const { return FilteredTriangles; }

		/** Create an array with all source triangles whose filter set from the given array containts the specified target vertex. */
		void GatherAllSourceTrianglesForTargetVertex(const TArray<FMeshToMeshFilterSet>& FilterSets, int32 TargetVertex) const;

		/** Find the closest triangles from the specified point. */
		CLOTHINGSYSTEMRUNTIMECOMMON_API TArray<int32> FindCandidateTriangles(const FVector& InPoint, float InTolerance = KINDA_SMALL_NUMBER) const;

		/**
		 * Return the max edge length for each edge coming out of a mesh vertex.
		 * Useful for guiding the search radius when searching for nearest triangles.
		 */
		CLOTHINGSYSTEMRUNTIMECOMMON_API TConstArrayView<float> GetMaxEdgeLengths() const;

	private:
		CLOTHINGSYSTEMRUNTIMECOMMON_API void ComputeAveragedNormals();

		struct FClothBvEntry;

		TConstArrayView<FVector3f> Positions;
		TConstArrayView<FVector3f> Normals;
		TConstArrayView<FVector3f> Tangents;
		TConstArrayView<uint32> Indices;

		TArray<FVector3f> AveragedNormals;

		mutable bool bHasValidBVH;
		mutable Chaos::TAABBTree<int32, Chaos::TAABBTreeLeafArray<int32, false>, false> BVH;

		mutable TArray<float> MaxEdgeLengths;
		mutable TArray<int32> FilteredTriangles;
	};

	/**
	 * Static method for calculating a skinned mesh result from source data
	 */
	void CLOTHINGSYSTEMRUNTIMECOMMON_API SkinPhysicsMesh(
		const TArray<int32>& BoneMap,  // UClothingAssetCommon::UsedBoneIndices
		const FClothPhysicalMeshData& InMesh, 
		const FTransform& PostTransform,  // Final transform to apply to component space positions and normals
		const FMatrix44f* InBoneMatrices, 
		const int32 InNumBoneMatrices, 
		TArray<FVector3f>& OutPositions, 
		TArray<FVector3f>& OutNormals);

	/**
	* Given mesh information for two meshes, generate a list of skinning data to embed TargetMesh in SourceMesh
	* 
	* @param OutMeshToMeshVertData      - Final skinning data
	* @param TargetMesh                 - Mesh data for the mesh we are embedding
	* @param SourceMesh                 - Mesh data for the mesh we are embedding into
	* @param MaxDistances               - Fixed positions mask used to update the vertex contributions, or null if the update is not required
	* @param bUseSmoothTransitions      - Set blend weight to smoothen transitions between the fixed and deformed vertices when updating the vertex contributions
	* @param bUseMultipleInfluences     - Whether to take a weighted average of influences from multiple source triangles
	* @param KernelMaxDistance          - Max distance parameter for weighting kernel for when using multiple influences
	* @param FilterSets                 - A set of source triangle indices to filter the search from, will use all source triangles if empty
	*/
	void CLOTHINGSYSTEMRUNTIMECOMMON_API GenerateMeshToMeshVertData(
		TArray<FMeshToMeshVertData>& OutMeshToMeshVertData,
		const ClothMeshDesc& TargetMesh,
		const ClothMeshDesc& SourceMesh,
		const FPointWeightMap* MaxDistances,
		bool bUseSmoothTransitions,
		bool bUseMultipleInfluences,
		float KernelMaxDistance,
		const TArray<FMeshToMeshFilterSet>& FilterSets = TArray<FMeshToMeshFilterSet>());

	/**
	 * Computes how much each vertex contributes to the final mesh. The final mesh is a blend
	 * between the cloth and the skinned mesh.
	 */
	void CLOTHINGSYSTEMRUNTIMECOMMON_API ComputeVertexContributions(
		TArray<FMeshToMeshVertData> &InOutSkinningData,
		const FPointWeightMap* const InMaxDistances,
		const bool bInSmoothTransition,
		const bool bInUseMultipleInfluences = false);

	/**
	* Given a triangle ABC with normals at each vertex NA, NB and NC, get a barycentric coordinate
	* and corresponding distance from the triangle encoded in an FVector4 where the components are
	* (BaryX, BaryY, BaryZ, Dist)
	* @param A		- Position of triangle vertex A
	* @param B		- Position of triangle vertex B
	* @param C		- Position of triangle vertex C
	* @param NA	- Normal at vertex A
	* @param NB	- Normal at vertex B
	* @param NC	- Normal at vertex C
	* @param Point	- Point to calculate Bary+Dist for
	*/
	FVector4f GetPointBaryAndDist(
		const FVector3f& A,
		const FVector3f& B,
		const FVector3f& C,
		const FVector3f& Point);

	/**
	* Given a triangle ABC with normals at each vertex NA, NB and NC, get a barycentric coordinate
	* and corresponding distance from the triangle encoded in an FVector4 where the components are
	* (BaryX, BaryY, BaryZ, Dist)
	* @param A		- Position of triangle vertex A
	* @param B		- Position of triangle vertex B
	* @param C		- Position of triangle vertex C
	* @param NA	- Normal at vertex A
	* @param NB	- Normal at vertex B
	* @param NC	- Normal at vertex C
	* @param Point	- Point to calculate Bary+Dist for
	*/
	FVector4f GetPointBaryAndDistWithNormals(
		const FVector3f& A,
		const FVector3f& B,
		const FVector3f& C,
		const FVector3f& NA,
		const FVector3f& NB,
		const FVector3f& NC,
		const FVector3f& Point);

	/** 
	 * Object used to map vertex parameters between two meshes using the
	 * same barycentric mesh to mesh mapping data we use for clothing
	 */
	class FVertexParameterMapper
	{
	public:
		UE_NONCOPYABLE(FVertexParameterMapper)

		FVertexParameterMapper(TConstArrayView<FVector3f> InMesh0Positions,
			TConstArrayView<FVector3f> InMesh0Normals,
			TConstArrayView<FVector3f> InMesh1Positions,
			TConstArrayView<FVector3f> InMesh1Normals,
			TConstArrayView<uint32> InMesh1Indices)
			: Mesh0Positions(InMesh0Positions)
			, Mesh0Normals(InMesh0Normals)
			, Mesh1Positions(InMesh1Positions)
			, Mesh1Normals(InMesh1Normals)
			, Mesh1Indices(InMesh1Indices)
		{
		}

		/** Generic mapping function, can be used to map any type with a provided callable */
		template<typename T, typename Lambda>
		void Map(TConstArrayView<T>& SourceData, TArray<T>& DestData, const Lambda& Func)
		{
			// Enforce the interp func signature (returns T and takes a bary and 3 Ts)
			// If you hit this then either the return type isn't T or your arguments aren't convertible to T
			static_assert(std::is_same_v<T, typename TDecay<decltype(Func(DeclVal<FVector3f>(), DeclVal<T>(), DeclVal<T>(), DeclVal<T>()))>::Type>, "Invalid Lambda signature passed to Map");

			const int32 NumMesh0Positions = Mesh0Positions.Num();
			const int32 NumMesh0Normals = Mesh0Normals.Num();

			const int32 NumMesh1Positions = Mesh1Positions.Num();
			const int32 NumMesh1Normals = Mesh1Normals.Num();
			const int32 NumMesh1Indices = Mesh1Indices.Num();

			// Validate mesh data
			check(NumMesh0Positions == NumMesh0Normals);
			check(NumMesh1Positions == NumMesh1Normals);
			check(NumMesh1Indices % 3 == 0);
			check(SourceData.Num() == NumMesh1Positions);

			if(DestData.Num() != NumMesh0Positions)
			{
				DestData.Reset();
				DestData.AddUninitialized(NumMesh0Positions);
			}

			TArray<FVector4> EmbeddedPositions;
			TArray<int32> SourceIndices;
			GenerateEmbeddedPositions(EmbeddedPositions, SourceIndices);

			for(int32 DestVertIndex = 0 ; DestVertIndex < NumMesh0Positions ; ++DestVertIndex)
			{
				// Truncate the distance from the position data
				FVector Bary = EmbeddedPositions[DestVertIndex];

				const int32 SourceTriBaseIdx = DestVertIndex * 3;
				T A = SourceData[SourceIndices[SourceTriBaseIdx + 0]];
				T B = SourceData[SourceIndices[SourceTriBaseIdx + 1]];
				T C = SourceData[SourceIndices[SourceTriBaseIdx + 2]];

				T& DestVal = DestData[DestVertIndex];

				// If we're super close to a vertex just take it's value.
				// Otherwise call the provided interp lambda
				FVector DiffVec = FVector::OneVector - Bary;
				if(FMath::Abs(DiffVec.X) <= SMALL_NUMBER)
				{
					DestVal = A;
				}
				else if(FMath::Abs(DiffVec.Y) <= SMALL_NUMBER)
				{
					DestVal = B;
				}
				else if(FMath::Abs(DiffVec.Z) <= SMALL_NUMBER)
				{
					DestVal = C;
				}
				else
				{
					DestVal = Func((FVector3f)Bary, A, B, C);
				}
			}
		}

		/**
		 * Remap float scalar values.
		 * This transfers the values from source to dest by matching the mesh topologies using
		 * barycentric coordinates of the dest mesh (Mesh0) positions over the source mesh (Mesh1).
		 */
		CLOTHINGSYSTEMRUNTIMECOMMON_API void Map(TConstArrayView<float> Source, TArray<float>& Dest);

	private:

		/** 
		 * Embeds a list of positions into a source mesh
		 * @param OutEmbeddedPositions Embedded version of the original positions, a barycentric coordinate and distance along the normal of the triangle
		 * @param OutSourceIndices Source index list for the embedded positions, 3 per position to denote the source triangle
		 */
		CLOTHINGSYSTEMRUNTIMECOMMON_API void GenerateEmbeddedPositions(TArray<FVector4>& OutEmbeddedPositions, TArray<int32>& OutSourceIndices);

		TConstArrayView<FVector3f> Mesh0Positions;
		TConstArrayView<FVector3f> Mesh0Normals;
		TConstArrayView<FVector3f> Mesh1Positions;
		TConstArrayView<FVector3f> Mesh1Normals;
		TConstArrayView<uint32> Mesh1Indices;
	};
}
